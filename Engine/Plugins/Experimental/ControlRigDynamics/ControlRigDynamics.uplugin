{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ControlRigDynamics",
	"Description": "Support for simple dynamics/cosmetic simulation in control rig",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"CanContainVerse": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "PhysicsControl",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "ControlRigDynamics",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ControlRigDynamicsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}