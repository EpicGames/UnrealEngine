// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ControlRigDynamics : ModuleRules
{
	public ControlRigDynamics(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		//OptimizeCode = CodeOptimization.Never;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"PhysicsControl"
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ControlRig",
				"CoreUObject",
				"Engine",
				"RigVM"
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"RigVMDeveloper",
					"ControlRigDeveloper",
					"Slate",
					"SlateCore",
					"EditorStyle",
				}
			);
		}
	}
}
