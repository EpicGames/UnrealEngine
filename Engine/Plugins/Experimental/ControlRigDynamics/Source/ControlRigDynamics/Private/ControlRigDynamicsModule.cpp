// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigDynamicsModule.h"
#include "RigDynamicsObjectVersion.h"

#include "UObject/DevObjectVersion.h"

#define LOCTEXT_NAMESPACE "FControlRigDynamicsModule"

DEFINE_LOG_CATEGORY(LogControlRigDynamics);


// Unique version id for serializing
const FGuid FRigDynamicsObjectVersion::GUID(0xA7C3E291, 0x5F8B4D06, 0x3E1A9C74, 0xD20F68B5);

// Register custom version with Core. 
FDevVersionRegistration GRegisterRigDynamicsObjectVersion(
	FRigDynamicsObjectVersion::GUID, FRigDynamicsObjectVersion::LatestVersion, TEXT("Dev-RigDynamics"));

//======================================================================================================================
void FControlRigDynamicsModule::StartupModule()
{
}

//======================================================================================================================
void FControlRigDynamicsModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FControlRigDynamicsModule, ControlRigDynamics)
