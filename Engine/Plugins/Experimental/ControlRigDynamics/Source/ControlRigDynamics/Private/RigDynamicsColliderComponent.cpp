// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsColliderComponent.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
void FRigDynamicsColliderComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	Ar << Shapes;
}

//======================================================================================================================
void FRigDynamicsColliderComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);

	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::HelperStructs)
	{
		Ar << Shapes;
	}
	else
	{
		Ar << Shapes.Boxes;
		Ar << Shapes.Capsules;
		Ar << Shapes.Planes;
	}
}
