// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsColliderExecution.h"

#include "RigDynamicsHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsColliderExecution)

//======================================================================================================================
FRigUnit_SpawnDynamicsCollider_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsCollider can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsColliderComponentKey = Controller->AddComponent(
			FRigDynamicsColliderComponent::StaticStruct(), ColliderComponentName, Owner);
		if (DynamicsColliderComponentKey.IsValid())
		{
			if (FRigDynamicsColliderComponent* Component =
				GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))
			{
				Component->Shapes.Boxes = Shapes.Boxes;
				Component->Shapes.Capsules = Shapes.Capsules;
				Component->Shapes.Planes = Shapes.Planes;
			}

			if (FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Solver->Colliders.Add(DynamicsColliderComponentKey);
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsColliderExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchySetDynamicsColliderShapes_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(
			TEXT("SetDynamicsColliderShapes can only be used during Setup; "
				 "use the per-shape Set nodes (by Name) at runtime."));
		return;
	}
	if (FRigDynamicsColliderComponent* Component = GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))
	{
		Component->Shapes = Shapes;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsColliderShapes_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsColliderComponent* Component =
		GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))
	{
		Shapes = Component->Shapes;
	}
}

//======================================================================================================================
// Macros to reduce boilerplate for the Set collider shape property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

// Sets a whole shape by matching on Name (e.g. ShapeType=Box, ShapeArray=Boxes)
#define IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE(ShapeType, ShapeArray)          \
FRigUnit_HierarchySetDynamicsCollider##ShapeType##_Execute()                  \
{                                                                             \
	if (!ExecuteContext.Hierarchy) { return; }                                \
	if (FRigDynamicsColliderComponent* Component =                            \
		GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey)) \
	{                                                                         \
		for (auto& Existing : Component->Shapes.ShapeArray)                   \
		{                                                                     \
			if (Existing.Name == ShapeType.Name) { Existing = ShapeType; }    \
		}                                                                     \
	}                                                                         \
}

// Sets an individual property on a shape found by Name
#define IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(ShapeType, ShapeArray, PropName) \
FRigUnit_HierarchySetDynamicsCollider##ShapeType##PropName##_Execute()              \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (FRigDynamicsColliderComponent* Component =                                  \
		GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))       \
	{                                                                               \
		for (auto& Shape : Component->Shapes.ShapeArray)                            \
		{                                                                           \
			if (Shape.Name == Name) { Shape.PropName = PropName; }                  \
		}                                                                           \
	}                                                                               \
}

// Gets a whole shape by matching on Name. The output pin (named after ShapeType, e.g. Box) is
// left at its struct default if no matching shape is found.
#define IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE(ShapeType, ShapeArray)                \
FRigUnit_HierarchyGetDynamicsCollider##ShapeType##_Execute()                        \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (const FRigDynamicsColliderComponent* Component =                            \
		GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))       \
	{                                                                               \
		for (const auto& Shape : Component->Shapes.ShapeArray)                      \
		{                                                                           \
			if (Shape.Name == Name) { ShapeType = Shape; break; }                   \
		}                                                                           \
	}                                                                               \
}

// Gets an individual property from a shape found by Name. The output pin (named after PropName)
// is left at its struct default if no matching shape is found.
#define IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(ShapeType, ShapeArray, PropName) \
FRigUnit_HierarchyGetDynamicsCollider##ShapeType##PropName##_Execute()              \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (const FRigDynamicsColliderComponent* Component =                            \
		GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))       \
	{                                                                               \
		for (const auto& Shape : Component->Shapes.ShapeArray)                      \
		{                                                                           \
			if (Shape.Name == Name) { PropName = Shape.PropName; break; }           \
		}                                                                           \
	}                                                                               \
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE(Box, Boxes)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE(Capsule, Capsules)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE(Plane, Planes)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE(Box, Boxes)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE(Capsule, Capsules)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE(Plane, Planes)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Box, Boxes, TM)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Box, Boxes, Extents)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Box, Boxes, TM)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Box, Boxes, Extents)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, TM)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, Radius)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, Length)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, TM)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, Radius)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Capsule, Capsules, Length)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Plane, Planes, TM)
IMPLEMENT_SET_DYNAMICS_COLLIDER_SHAPE_PROP(Plane, Planes, Extents)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Plane, Planes, TM)
IMPLEMENT_GET_DYNAMICS_COLLIDER_SHAPE_PROP(Plane, Planes, Extents)
