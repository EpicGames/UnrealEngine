// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConeLimitComponent.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
void FRigDynamicsConeLimitComponent::Serialize(FArchive& Ar)
{
	Ar << GrandparentComponentKey;
	Ar << ParentComponentKey;
	Ar << ChildComponentKey;
	Ar << Strength;
	Ar << DampingRatio;
	Ar << Angle;
}

//======================================================================================================================
void FRigDynamicsConeLimitComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);
	FRigBaseComponent::Save(Ar);
	Serialize(Ar);
}

//======================================================================================================================
void FRigDynamicsConeLimitComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Serialize(Ar);
}
