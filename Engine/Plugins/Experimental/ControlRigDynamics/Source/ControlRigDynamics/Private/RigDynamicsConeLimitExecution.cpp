// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConeLimitExecution.h"

#include "RigDynamicsHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsConeLimitExecution)

//======================================================================================================================
// Macros to reduce boilerplate for the Get/Set cone limit property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

#define IMPLEMENT_SET_DYNAMICS_CONE_LIMIT(PropName)                               \
FRigUnit_HierarchySetDynamicsConeLimit##PropName##_Execute()                      \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (FRigDynamicsConeLimitComponent* Component =                               \
		GetConeLimit(*ExecuteContext.Hierarchy, DynamicsConeLimitComponentKey))   \
	{                                                                             \
		Component->PropName = PropName;                                           \
	}                                                                             \
}

#define IMPLEMENT_GET_DYNAMICS_CONE_LIMIT(PropName)                               \
FRigUnit_HierarchyGetDynamicsConeLimit##PropName##_Execute()                      \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (const FRigDynamicsConeLimitComponent* Component =                         \
		GetConeLimit(*ExecuteContext.Hierarchy, DynamicsConeLimitComponentKey))   \
	{                                                                             \
		PropName = Component->PropName;                                           \
	}                                                                             \
}

//======================================================================================================================
FRigUnit_SpawnDynamicsConeLimit_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsConeLimit can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsConeLimitComponentKey = Controller->AddComponent(
			FRigDynamicsConeLimitComponent::StaticStruct(), ConeLimitComponentName, Owner);
		if (DynamicsConeLimitComponentKey.IsValid())
		{
			if (FRigDynamicsConeLimitComponent* Component =
				GetConeLimit(*ExecuteContext.Hierarchy, DynamicsConeLimitComponentKey))
			{
				Component->GrandparentComponentKey = GrandparentComponentKey;
				Component->ParentComponentKey = ParentComponentKey;
				Component->ChildComponentKey = ChildComponentKey;
				Component->Strength = Strength;
				Component->DampingRatio = DampingRatio;
				Component->Angle = Angle;
			}

			if (FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Solver->ConeLimits.Add(DynamicsConeLimitComponentKey);
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsConeLimitExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetConeLimit(*ExecuteContext.Hierarchy, DynamicsConeLimitComponentKey) != nullptr;
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONE_LIMIT(Strength)
IMPLEMENT_GET_DYNAMICS_CONE_LIMIT(Strength)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONE_LIMIT(DampingRatio)
IMPLEMENT_GET_DYNAMICS_CONE_LIMIT(DampingRatio)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONE_LIMIT(Angle)
IMPLEMENT_GET_DYNAMICS_CONE_LIMIT(Angle)
