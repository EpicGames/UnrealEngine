// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConfinerComponent.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
void FRigDynamicsConfinerComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);

	FRigBaseComponent::Save(Ar);
	Ar << Shapes;
	Ar << Strength;
}

//======================================================================================================================
void FRigDynamicsConfinerComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);

	Ar << Shapes;
	Ar << Strength;
}
