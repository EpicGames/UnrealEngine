// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConfinerExecution.h"

#include "RigDynamicsHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsConfinerExecution)

//======================================================================================================================
FRigUnit_SpawnDynamicsConfiner_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsConfiner can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsConfinerComponentKey = Controller->AddComponent(
			FRigDynamicsConfinerComponent::StaticStruct(), ConfinerComponentName, Owner);
		if (DynamicsConfinerComponentKey.IsValid())
		{
			if (FRigDynamicsConfinerComponent* Component =
				GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))
			{
				Component->Shapes.Boxes = Shapes.Boxes;
				Component->Shapes.Capsules = Shapes.Capsules;
				Component->Shapes.Planes = Shapes.Planes;
				Component->Strength = Strength;
			}

			if (FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Solver->Confiners.Add(DynamicsConfinerComponentKey);
			}

			// Opt the listed particles in to confinement by this new confiner. Equivalent to
			// calling HierarchyEnableDynamicsConfinementWithConfiner once per particle.
			for (const FRigComponentKey& ParticleKey : ConfinedParticles)
			{
				if (FRigDynamicsParticleComponent* ParticleComponent =
					GetParticle(*ExecuteContext.Hierarchy, ParticleKey))
				{
					ParticleComponent->ParticleProperties.Confiners.AddUnique(DynamicsConfinerComponentKey);
				}
				else
				{
					UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(
						TEXT("SpawnDynamicsConfiner: Unable to find confined particle"));
				}
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsConfinerExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchySetDynamicsConfinerShapes_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(
			TEXT("SetDynamicsConfinerShapes can only be used during Setup; "
				 "use the per-shape Set nodes (by Name) at runtime."));
		return;
	}
	if (FRigDynamicsConfinerComponent* Component = GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))
	{
		Component->Shapes = Shapes;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsConfinerShapes_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsConfinerComponent* Component =
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))
	{
		Shapes = Component->Shapes;
	}
}

//======================================================================================================================
// Macros to reduce boilerplate for the Set confiner shape property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

// Sets a whole shape by matching on Name (e.g. ShapeType=Box, ShapeArray=Boxes)
#define IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE(ShapeType, ShapeArray)                \
FRigUnit_HierarchySetDynamicsConfiner##ShapeType##_Execute()                        \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (FRigDynamicsConfinerComponent* Component =                                  \
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))       \
	{                                                                               \
		for (auto& Existing : Component->Shapes.ShapeArray)                         \
		{                                                                           \
			if (Existing.Name == ShapeType.Name) { Existing = ShapeType; }          \
		}                                                                           \
	}                                                                               \
}

// Sets an individual property on a shape found by Name
#define IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(ShapeType, ShapeArray, PropName) \
FRigUnit_HierarchySetDynamicsConfiner##ShapeType##PropName##_Execute()              \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (FRigDynamicsConfinerComponent* Component =                                  \
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))       \
	{                                                                               \
		for (auto& Shape : Component->Shapes.ShapeArray)                            \
		{                                                                           \
			if (Shape.Name == Name) { Shape.PropName = PropName; }                  \
		}                                                                           \
	}                                                                               \
}

// Gets a whole shape by matching on Name. The output pin (named after ShapeType, e.g. Box) is
// left at its struct default if no matching shape is found.
#define IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE(ShapeType, ShapeArray)                \
FRigUnit_HierarchyGetDynamicsConfiner##ShapeType##_Execute()                        \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (const FRigDynamicsConfinerComponent* Component =                            \
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))       \
	{                                                                               \
		for (const auto& Shape : Component->Shapes.ShapeArray)                      \
		{                                                                           \
			if (Shape.Name == Name) { ShapeType = Shape; break; }                   \
		}                                                                           \
	}                                                                               \
}

// Gets an individual property from a shape found by Name. The output pin (named after PropName)
// is left at its struct default if no matching shape is found.
#define IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(ShapeType, ShapeArray, PropName) \
FRigUnit_HierarchyGetDynamicsConfiner##ShapeType##PropName##_Execute()              \
{                                                                                   \
	if (!ExecuteContext.Hierarchy) { return; }                                      \
	if (const FRigDynamicsConfinerComponent* Component =                            \
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))       \
	{                                                                               \
		for (const auto& Shape : Component->Shapes.ShapeArray)                      \
		{                                                                           \
			if (Shape.Name == Name) { PropName = Shape.PropName; break; }           \
		}                                                                           \
	}                                                                               \
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE(Box, Boxes)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE(Capsule, Capsules)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE(Plane, Planes)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE(Box, Boxes)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE(Capsule, Capsules)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE(Plane, Planes)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Box, Boxes, TM)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Box, Boxes, Extents)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Box, Boxes, TM)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Box, Boxes, Extents)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, TM)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, Radius)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, Length)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, TM)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, Radius)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Capsule, Capsules, Length)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Plane, Planes, TM)
IMPLEMENT_SET_DYNAMICS_CONFINER_SHAPE_PROP(Plane, Planes, Extents)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Plane, Planes, TM)
IMPLEMENT_GET_DYNAMICS_CONFINER_SHAPE_PROP(Plane, Planes, Extents)

//======================================================================================================================
FRigUnit_HierarchySetDynamicsConfinerStrength_Execute()
{
	if (!ExecuteContext.Hierarchy) 
	{
		return; 
	}
	if (FRigDynamicsConfinerComponent* Component = GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))
	{
		Component->Strength = Strength;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsConfinerStrength_Execute()
{
	if (!ExecuteContext.Hierarchy) { 
		return; 
	}
	if (const FRigDynamicsConfinerComponent* Component = 
		GetConfiner(*ExecuteContext.Hierarchy, DynamicsConfinerComponentKey))
	{
		Strength = Component->Strength;
	}
}
