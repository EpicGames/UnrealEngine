// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConstraintComponent.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
void FRigDynamicsConstraintComponent::Serialize(FArchive& Ar)
{
	Ar << ParentComponentKey;
	Ar << ChildComponentKey;
	Ar << ConstraintType;
	Ar << Strength;
	Ar << DampingRatio;
	Ar << ExtraDamping;
	Ar << LengthMultiplier;
	Ar << ExtraLength;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::AccelerationModeAndDamping)
	{
		Ar << bAccelerationMode;
	}
	else if (Ar.IsLoading())
	{
		// Preserve current behaviour for legacy saves: the existing XPBD math is already accel mode.
		bAccelerationMode = true;
	}
}

//======================================================================================================================
void FRigDynamicsConstraintComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);
	FRigBaseComponent::Save(Ar);
	Serialize(Ar);
}

//======================================================================================================================
void FRigDynamicsConstraintComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Serialize(Ar);
}
