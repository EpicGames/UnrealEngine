// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsConstraintExecution.h"

#include "RigDynamicsHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsConstraintExecution)

//======================================================================================================================
// Macros to reduce boilerplate for the Get/Set constraint property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

#define IMPLEMENT_SET_DYNAMICS_CONSTRAINT(PropName)                               \
FRigUnit_HierarchySetDynamicsConstraint##PropName##_Execute()                     \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (FRigDynamicsConstraintComponent* Component =                              \
		GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey)) \
	{                                                                             \
		Component->PropName = PropName;                                           \
	}                                                                             \
}

#define IMPLEMENT_GET_DYNAMICS_CONSTRAINT(PropName)                               \
FRigUnit_HierarchyGetDynamicsConstraint##PropName##_Execute()                     \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (const FRigDynamicsConstraintComponent* Component =                        \
		GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey)) \
	{                                                                             \
		PropName = Component->PropName;                                           \
	}                                                                             \
}

//======================================================================================================================
FRigUnit_SpawnDynamicsConstraint_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsConstraint can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsConstraintComponentKey = Controller->AddComponent(
			FRigDynamicsConstraintComponent::StaticStruct(), ConstraintComponentName, Owner);
		if (DynamicsConstraintComponentKey.IsValid())
		{
			if (FRigDynamicsConstraintComponent* Component =
				GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey))
			{
				Component->ParentComponentKey = ParentComponentKey;
				Component->ChildComponentKey = ChildComponentKey;
				Component->ConstraintType = ConstraintType;
				Component->Strength = Strength;
				Component->DampingRatio = DampingRatio;
				Component->ExtraDamping = ExtraDamping;
				Component->bAccelerationMode = bAccelerationMode;
				Component->LengthMultiplier = LengthMultiplier;
				Component->ExtraLength = ExtraLength;
			}

			if (FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Solver->Constraints.Add(DynamicsConstraintComponentKey);
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsConstraintExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey) != nullptr;
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONSTRAINT(Strength)
IMPLEMENT_GET_DYNAMICS_CONSTRAINT(Strength)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONSTRAINT(DampingRatio)
IMPLEMENT_GET_DYNAMICS_CONSTRAINT(DampingRatio)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONSTRAINT(ExtraDamping)
IMPLEMENT_GET_DYNAMICS_CONSTRAINT(ExtraDamping)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONSTRAINT(LengthMultiplier)
IMPLEMENT_GET_DYNAMICS_CONSTRAINT(LengthMultiplier)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_CONSTRAINT(ExtraLength)
IMPLEMENT_GET_DYNAMICS_CONSTRAINT(ExtraLength)

//======================================================================================================================
// Written out rather than using the Set/Get macros because the field name (bAccelerationMode) does
// not match its sanitised macro tail (AccelerationMode).
//======================================================================================================================

//======================================================================================================================
FRigUnit_HierarchySetDynamicsConstraintAccelerationMode_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsConstraintComponent* Component =
		GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey))
	{
		Component->bAccelerationMode = bAccelerationMode;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsConstraintAccelerationMode_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsConstraintComponent* Component =
		GetConstraint(*ExecuteContext.Hierarchy, DynamicsConstraintComponentKey))
	{
		bAccelerationMode = Component->bAccelerationMode;
	}
}
