// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsData.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsParticleProperties& Data)
{
	Ar << Data.Radius;
	Ar << Data.Mass;
	Ar << Data.MovementType;
	Ar << Data.GravityMultiplier;
	Ar << Data.Strength;
	Ar << Data.DampingRatio;
	Ar << Data.ExtraDamping;
	Ar << Data.TargetVelocityInfluence;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::TargetModeFloat)
	{
		Ar << Data.TargetMode;
	}
	else
	{
		// Ignore old data (we have no legacy assets)
		uint8 LegacyTargetMode = 1;
		Ar << LegacyTargetMode;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::AngleLimit)
	{
		Ar << Data.AngleLimit;
		Ar << Data.AngleLimitStrength;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::NoCollisionColliders)
	{
		Ar << Data.NoCollisionColliders;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::CollisionParticles)
	{
		Ar << Data.CollisionParticles;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::CollideWithColliders)
	{
		Ar << Data.bCollideWithColliders;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::ParticleDrag)
	{
		// Field was renamed from Drag to Damping in AccelerationModeAndDamping; on-disk bytes unchanged.
		Ar << Data.Damping;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::Confiners)
	{
		Ar << Data.Confiners;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::AccelerationModeAndDamping)
	{
		Ar << Data.bAccelerationMode;
		Ar << Data.bScaleDampingByInverseMass;
	}
	else if (Ar.IsLoading())
	{
		// Preserve current behaviour for legacy saves: the existing XPBD math is already acceleration
		// mode, and the legacy Drag field was always mass-scaled.
		Data.bAccelerationMode = true;
		Data.bScaleDampingByInverseMass = true;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeBox& Data)
{
	Ar << Data.Name;
	Ar << Data.TM;
	Ar << Data.Extents;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeCapsule& Data)
{
	Ar << Data.Name;
	Ar << Data.TM;
	Ar << Data.Radius;
	Ar << Data.Length;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsShapePlane& Data)
{
	Ar << Data.Name;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::PlaneDefinition)
	{
		Ar << Data.TM;
	}
	else
	{
		FVector Junk;
		Ar << Junk;
		Ar << Junk;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::PlaneExtents)
	{
		Ar << Data.Extents;
	}
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeCollection& Data)
{
	Ar << Data.Boxes;
	Ar << Data.Capsules;
	Ar << Data.Planes;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsSimulationDragSettings& Data)
{
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::SimulationAdditionalDrag)
	{
		// Field was renamed from AdditionalDrag to AdditionalDamping in AccelerationModeAndDamping;
		// on-disk bytes unchanged.
		Ar << Data.AdditionalDamping;
	}
	// Older saves leave AdditionalDamping at the constructor default of 0 (no global drag baseline).
	Ar << Data.LinearDragMultiplier;
	Ar << Data.AngularDragMultiplier;
	Ar << Data.ExternalLinearVelocity;
	Ar << Data.ExternalAngularVelocity;
	Ar << Data.ExternalTurbulenceVelocity;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsInertialForceSettings& Data)
{
	Ar << Data.Amount;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::InertialForcePerTermAmounts)
	{
		Ar << Data.LinearEulerAmount;
		Ar << Data.AngularEulerAmount;
		Ar << Data.CentrifugalAmount;
		Ar << Data.CoriolisAmount;
	}
	// Older saves leave the per-term multipliers at their constructor default of 1.0, which
	// reproduces the legacy single-knob behaviour exactly.
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsSimulationSpaceMotion& Data)
{
	Ar << Data.VerticalMotionScale;
	Ar << Data.bClampLinearVelocity;
	Ar << Data.MaxLinearVelocity;
	Ar << Data.bClampAngularVelocity;
	Ar << Data.MaxAngularVelocity;
	Ar << Data.bClampLinearAcceleration;
	Ar << Data.MaxLinearAcceleration;
	Ar << Data.bClampAngularAcceleration;
	Ar << Data.MaxAngularAcceleration;
	Ar << Data.InertialForces;
	Ar << Data.Drag;
	return Ar;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsTeleportDetectionSettings& Data)
{
	Ar << Data.bFromPositionChange;
	Ar << Data.PositionChangeThreshold;
	Ar << Data.bFromOrientationChange;
	Ar << Data.OrientationChangeThreshold;
	Ar << Data.bFromLinearAcceleration;
	Ar << Data.LinearAccelerationThreshold;
	Ar << Data.bFromAngularAcceleration;
	Ar << Data.AngularAccelerationThreshold;
	return Ar;
}

//======================================================================================================================
// Drains the bytes of a pre-SimulationSpaceRegrouping FRigDynamicsSimulationSpaceSettings record.
// All values are discarded. Mirrors the layout the old operator<< wrote so subsequent fields in
// the parent archive stay aligned.
void DrainLegacySimulationSpaceSettings(FArchive& Ar)
{
	float   LegacySpaceMovementAmount = 0.0f;
	float   LegacyVelocityScaleZ = 0.0f;
	bool    LegacyBool = false;
	float   LegacyFloat = 0.0f;

	Ar << LegacySpaceMovementAmount;
	Ar << LegacyVelocityScaleZ;
	Ar << LegacyBool; Ar << LegacyFloat;   // bClampLinearVelocity      / MaxLinearVelocity
	Ar << LegacyBool; Ar << LegacyFloat;   // bClampAngularVelocity     / MaxAngularVelocity
	Ar << LegacyBool; Ar << LegacyFloat;   // bClampLinearAcceleration  / MaxLinearAcceleration
	Ar << LegacyBool; Ar << LegacyFloat;   // bClampAngularAcceleration / MaxAngularAcceleration
	Ar << LegacyFloat;                     // LinearAccelerationThresholdForTeleport
	Ar << LegacyFloat;                     // AngularAccelerationThresholdForTeleport
	Ar << LegacyFloat;                     // PositionChangeThresholdForTeleport
	Ar << LegacyFloat;                     // OrientationChangeThresholdForTeleport
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::TeleportFlags)
	{
		Ar << LegacyBool;                  // bTeleportFromLinearAcceleration
		Ar << LegacyBool;                  // bTeleportFromAngularAcceleration
		Ar << LegacyBool;                  // bTeleportFromPositionChange
		Ar << LegacyBool;                  // bTeleportFromOrientationChange
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) < FRigDynamicsObjectVersion::DragSettings)
	{
		// Even older: drag fields were inside FRigDynamicsSimulationSpaceSettings.
		FVector LegacyVector;
		Ar << LegacyFloat;                 // LegacyLinearDragMultiplier
		Ar << LegacyFloat;                 // LegacyAngularDragMultiplier
		Ar << LegacyVector;                // LegacyExternalLinearDrag
		Ar << LegacyVector;                // LegacyExternalLinearVelocity
		Ar << LegacyVector;                // LegacyExternalAngularVelocity
		Ar << LegacyVector;                // LegacyExternalTurbulenceVelocity
	}
}

//======================================================================================================================
// Translates the legacy combined SimulationSpaceSettings struct into the new SpaceMotion +
// TeleportDetection. The legacy SpaceMovementAmount was a single master scalar on all four
// inertial pseudo-forces, which now corresponds to FRigDynamicsInertialForceSettings::Amount; the
// per-term gains are left at their constructor default of 1.0 so the legacy behaviour is preserved
// exactly. SpaceMotion.Drag is intentionally untouched - in the legacy pin layout drag came from
// a separate FRigDynamicsSimulationDragSettings pin and the caller assigns it directly.
void ConvertLegacyDynamicsSimulationSpaceSettings(
	const FRigDynamicsSimulationSpaceSettings& In,
	FRigDynamicsSimulationSpaceMotion&         OutMotion,
	FRigDynamicsTeleportDetectionSettings&     OutTeleport)
{
	OutMotion.VerticalMotionScale          = In.VelocityScaleZ;
	OutMotion.bClampLinearVelocity         = In.bClampLinearVelocity;
	OutMotion.MaxLinearVelocity            = In.MaxLinearVelocity;
	OutMotion.bClampAngularVelocity        = In.bClampAngularVelocity;
	OutMotion.MaxAngularVelocity           = In.MaxAngularVelocity;
	OutMotion.bClampLinearAcceleration     = In.bClampLinearAcceleration;
	OutMotion.MaxLinearAcceleration        = In.MaxLinearAcceleration;
	OutMotion.bClampAngularAcceleration    = In.bClampAngularAcceleration;
	OutMotion.MaxAngularAcceleration       = In.MaxAngularAcceleration;
	OutMotion.InertialForces.Amount        = In.SpaceMovementAmount;

	OutTeleport.bFromLinearAcceleration       = In.bTeleportFromLinearAcceleration;
	OutTeleport.LinearAccelerationThreshold   = In.LinearAccelerationThresholdForTeleport;
	OutTeleport.bFromAngularAcceleration      = In.bTeleportFromAngularAcceleration;
	OutTeleport.AngularAccelerationThreshold  = In.AngularAccelerationThresholdForTeleport;
	OutTeleport.bFromPositionChange           = In.bTeleportFromPositionChange;
	OutTeleport.PositionChangeThreshold       = In.PositionChangeThresholdForTeleport;
	OutTeleport.bFromOrientationChange        = In.bTeleportFromOrientationChange;
	OutTeleport.OrientationChangeThreshold    = In.OrientationChangeThresholdForTeleport;
}

//======================================================================================================================
FArchive& operator <<(FArchive& Ar, FRigDynamicsSolverSettings& Data)
{
	Ar << Data.Gravity;
	Ar << Data.MaxTimeStep;
	Ar << Data.MaxNumSteps;
	Ar << Data.NumIterations;
	Ar << Data.NumConstraintSubIterations;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::BonePositionAndOrientationSetting)
	{
		Ar << Data.bReadBoneOrientations;
		Ar << Data.bReadBonePositions;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::SimulationSpace)
	{
		Ar << Data.SimulationSpace;
		if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) < FRigDynamicsObjectVersion::RemoveSolverLevelColliders)
		{
			// Legacy: CollisionSpace + solver-level Colliders were removed. Drain the bytes so
			// subsequent fields stay aligned; the values are discarded.
			ERigDynamicsSimulationSpace LegacyCollisionSpace = ERigDynamicsSimulationSpace::Component;
			Ar << LegacyCollisionSpace;
		}
		Ar << Data.SpaceBone;
		if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) < FRigDynamicsObjectVersion::RemoveSolverLevelColliders)
		{
			FRigDynamicsShapeCollection LegacyColliders;
			Ar << LegacyColliders;
		}
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::ResetDetection)
	{
		Ar << Data.PositionThresholdForReset;
		Ar << Data.KinematicSpeedThresholdForReset;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::ResetFlags)
	{
		Ar << Data.bResetFromPosition;
		Ar << Data.bResetFromKinematicSpeed;
	}
	else if (Ar.IsLoading())
	{
		// Pre-ResetFlags the gate was "threshold > 0" - infer the toggle from the loaded value.
		Data.bResetFromPosition       = Data.PositionThresholdForReset       > 0.0f;
		Data.bResetFromKinematicSpeed = Data.KinematicSpeedThresholdForReset > 0.0f;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::EvaluationIntervalReset)
	{
		Ar << Data.bResetFromEvaluationInterval;
		Ar << Data.EvaluationIntervalThresholdForReset;
	}
	return Ar;
}
