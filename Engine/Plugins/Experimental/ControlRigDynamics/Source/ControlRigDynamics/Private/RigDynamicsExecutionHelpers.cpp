// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsExecutionHelpers.h"

#include "RigDynamicsHelpers.h"
#include "RigDynamicsSolver.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "PhysicsEngine/PhysicsConstraintTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsExecutionHelpers)

//======================================================================================================================
// Returns a parallel array of [0,1] chain positions for each bone in BoneKeys: 0 at chain roots,
// 1 at leaves, monotonically increasing parent->child, linear along the longest root-to-leaf path
// in each connected sub-forest. "Parent" means the nearest ancestor bone that is also in BoneKeys.
static TArray<float> ComputeBoneChainPositions(
	const URigHierarchy& Hierarchy, const TArray<FRigElementKey>& BoneKeys)
{
	const int32 NumBones = BoneKeys.Num();

	TArray<float> ChainPositions;
	ChainPositions.SetNumZeroed(NumBones);
	if (NumBones == 0)
	{
		return ChainPositions;
	}

	// 1. Map bone key -> index in BoneKeys.
	TMap<FRigElementKey, int32> IndexInSet;
	IndexInSet.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		IndexInSet.Add(BoneKeys[Index], Index);
	}

	// 2. Resolve each bone's parent within the set by walking up the hierarchy.
	TArray<int32> ParentIndices;
	ParentIndices.Init(INDEX_NONE, NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		const FRigBaseElement* Element = Hierarchy.Find(BoneKeys[Index]);
		const FRigBaseElement* Ancestor = Element ? Hierarchy.GetFirstParent(Element) : nullptr;
		while (Ancestor)
		{
			if (const int32* Found = IndexInSet.Find(Ancestor->GetKey()))
			{
				ParentIndices[Index] = *Found;
				break;
			}
			Ancestor = Hierarchy.GetFirstParent(Ancestor);
		}
	}

	// 3. Build children adjacency list.
	TArray<TArray<int32>> ChildrenIndices;
	ChildrenIndices.SetNum(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		const int32 Parent = ParentIndices[Index];
		if (Parent != INDEX_NONE)
		{
			ChildrenIndices[Parent].Add(Index);
		}
	}

	// 4. BFS from every root to produce a root-to-leaf topological order.
	TArray<int32> TopoOrder;
	TopoOrder.Reserve(NumBones);
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		if (ParentIndices[Index] == INDEX_NONE)
		{
			TopoOrder.Add(Index);
		}
	}
	for (int32 Head = 0; Head < TopoOrder.Num(); ++Head)
	{
		const int32 Node = TopoOrder[Head];
		for (int32 Child : ChildrenIndices[Node])
		{
			TopoOrder.Add(Child);
		}
	}
	check(TopoOrder.Num() == NumBones);

	// 5. Forward pass: depth from the chain root.
	TArray<int32> Depth;
	Depth.SetNumZeroed(NumBones);
	for (int32 Index : TopoOrder)
	{
		const int32 Parent = ParentIndices[Index];
		Depth[Index] = (Parent == INDEX_NONE) ? 0 : Depth[Parent] + 1;
	}

	// 6. Reverse pass: max distance from each bone down to any leaf in its subtree.
	TArray<int32> MaxToLeaf;
	MaxToLeaf.SetNumZeroed(NumBones);
	for (int32 OrderPos = TopoOrder.Num() - 1; OrderPos >= 0; --OrderPos)
	{
		const int32 Index = TopoOrder[OrderPos];
		const int32 Parent = ParentIndices[Index];
		if (Parent != INDEX_NONE)
		{
			MaxToLeaf[Parent] = FMath::Max(MaxToLeaf[Parent], MaxToLeaf[Index] + 1);
		}
	}

	// 7. Normalize by the length of the longest root-to-leaf path passing through each bone.
	for (int32 Index = 0; Index < NumBones; ++Index)
	{
		const int32 Total = Depth[Index] + MaxToLeaf[Index];
		ChainPositions[Index] = (Total > 0) ? (float(Depth[Index]) / float(Total)) : 0.0f;
	}

	return ChainPositions;
}

//======================================================================================================================
FRigUnit_SpawnDynamicsChains_Execute()
{
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsChains can only be used during Setup"));
		return;
	}

	// The construction event can be re-entered in preview without re-instantiating the unit, so
	// wipe any output arrays from prior runs before we start appending fresh keys.
	ParticleComponentKeys.Reset();
	ColliderComponentKeys.Reset();

	if (!ExecuteContext.Hierarchy)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsChains: No Hierarchy"));
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController();
	if (!Controller)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsChains: No Controller"));
		return;
	}

	FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());

	// Get the solver if it exists - it's OK to continue even if it exists (the user can assign
	// components later).
	FRigDynamicsSolverComponent* SolverComponent = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey);

	// The array of elements for which we want to create particles 
	TArray<FRigElementKey> ParticleElementKeys;

	TSet<FRigElementKey> ExcludeElementKeys;
	for (const FRigElementKey& ElementKey : ChainTerminatorBones)
	{
		TArray<FRigElementKey> Children = Hierarchy.GetChildren(ElementKey, true);
		ExcludeElementKeys.Append(Children);
	}

	// The solver will sort constraints and particles, so we don't need to worry about chains that overlap.
	for (const FRigElementKey& ElementKey : RootBones)
	{
		ParticleElementKeys.Add(ElementKey);

		TArray<FRigElementKey> Children = Hierarchy.GetChildren(ElementKey, true);

		for (const FRigElementKey& Child : Children)
		{
			if (!ExcludeElementKeys.Contains(Child))
			{
				ParticleElementKeys.AddUnique(Child);
			}
		}
	}

	// Apply the masks
	if (!ElementIncludeMask.IsEmpty())
	{
		ParticleElementKeys.RemoveAll(
			[&ElementIncludeMask](const FRigElementKey& Key) { return ElementIncludeMask.Find(Key) == INDEX_NONE; });
	}
	ParticleElementKeys.RemoveAll(
		[&ElementExcludeMask](const FRigElementKey& Key) { return ElementExcludeMask.Find(Key) != INDEX_NONE; });

	// Add extras _after_ any exclusions above
	for (const FRigElementKey& ElementKey : ExtraBones)
	{
		ParticleElementKeys.AddUnique(ElementKey);
	}

	// Now we have ParticleElementKeys

	// Compute along-chain position for each bone in ParticleElementKeys: 0 at chain roots, 1 at
	// leaves, monotonically increasing parent->child. Parallel to ParticleElementKeys. Used to
	// evaluate per-particle multiplier curves below.
	const TArray<float> ChainPositions = ComputeBoneChainPositions(Hierarchy, ParticleElementKeys);

	const FRichCurve* RadiusCurve             = ChainCurves.RadiusMultiplier.GetRichCurveConst();
	const FRichCurve* MassCurve               = ChainCurves.MassMultiplier.GetRichCurveConst();
	const FRichCurve* GravityCurve            = ChainCurves.GravityMultiplier.GetRichCurveConst();
	const FRichCurve* StrengthCurve           = ChainCurves.StrengthMultiplier.GetRichCurveConst();
	const FRichCurve* DampingRatioCurve       = ChainCurves.DampingRatioMultiplier.GetRichCurveConst();
	const FRichCurve* ExtraDampingCurve       = ChainCurves.ExtraDampingMultiplier.GetRichCurveConst();
	const FRichCurve* DampingCurve            = ChainCurves.DampingMultiplier.GetRichCurveConst();
	const FRichCurve* TargetModeCurve         = ChainCurves.TargetModeMultiplier.GetRichCurveConst();
	const FRichCurve* AngleLimitCurve         = ChainCurves.AngleLimitMultiplier.GetRichCurveConst();
	const FRichCurve* AngleLimitStrengthCurve = ChainCurves.AngleLimitStrengthMultiplier.GetRichCurveConst();

	for (int32 ParticleIndex = 0; ParticleIndex < ParticleElementKeys.Num(); ++ParticleIndex)
	{
		const FRigElementKey& ElementKey = ParticleElementKeys[ParticleIndex];
		FRigComponentKey DynamicsParticleComponentKey = Controller->AddComponent(
			FRigDynamicsParticleComponent::StaticStruct(), ParticleComponentName, ElementKey);
		if (DynamicsParticleComponentKey.IsValid())
		{
			if (FRigDynamicsParticleComponent* Component =
				GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
			{
				Component->ParticleProperties = ParticleProperties;

				const float ChainPos = ChainPositions[ParticleIndex];
				Component->ParticleProperties.Radius             *= RadiusCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.Mass               *= MassCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.GravityMultiplier  *= GravityCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.Strength           *= StrengthCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.DampingRatio       *= DampingRatioCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.ExtraDamping       *= ExtraDampingCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.Damping            *= DampingCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.TargetMode         *= TargetModeCurve->Eval(ChainPos, 1.0f);
				Component->ParticleProperties.AngleLimit         *= FMath::Clamp(
					AngleLimitCurve->Eval(ChainPos, 1.0f), 0.0f, 180.0f);
				Component->ParticleProperties.AngleLimitStrength *= AngleLimitStrengthCurve->Eval(ChainPos, 1.0f);
			}

			if (SolverComponent)
			{
				SolverComponent->Particles.Add(DynamicsParticleComponentKey);
			}
			ParticleComponentKeys.Add(DynamicsParticleComponentKey);
		}
	}

	for (const FRigDynamicsShapeCollectionWithKey& ShapeCollectionWithKey : Colliders)
	{
		const FRigElementKey& OwnerKey = ShapeCollectionWithKey.Owner;
		const FRigDynamicsShapeCollection& ShapeCollection = ShapeCollectionWithKey.Shapes;

		FRigComponentKey ColliderComponentKey = Controller->AddComponent(
			FRigDynamicsColliderComponent::StaticStruct(), ColliderComponentName, OwnerKey);
		if (ColliderComponentKey.IsValid())
		{
			if (FRigDynamicsColliderComponent* Component = GetCollider(*ExecuteContext.Hierarchy, ColliderComponentKey))
			{
				Component->Shapes.Boxes = ShapeCollection.Boxes;
				Component->Shapes.Capsules = ShapeCollection.Capsules;
				Component->Shapes.Planes = ShapeCollection.Planes;
			}

			if (SolverComponent)
			{
				SolverComponent->Colliders.Add(ColliderComponentKey);
			}

			ColliderComponentKeys.Add(ColliderComponentKey);
		}
	}
}

//======================================================================================================================
FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset_Execute()
{
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset can only be used during Setup"));
		return;
	}

	// The construction event can be re-entered in preview without re-instantiating the unit, so
	// wipe the output array from prior runs before we start appending fresh keys.
	DynamicsColliderComponentKeys.Reset();

	if (!ExecuteContext.Hierarchy)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset: No Hierarchy"));
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	if (!PhysicsAsset)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset needs a valid physics asset"));
		return;
	}

	URigHierarchyController* Controller = Hierarchy.GetController();
	if (!Controller)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset: No Controller"));
		return;
	}

	FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey);

	FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());

	for (TObjectPtr<USkeletalBodySetup> SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (!SkeletalBodySetup)
		{
			continue;
		}

		// Check we can use this based on the bone name
		FName BoneName = SkeletalBodySetup->BoneName;
		const FRigElementKey OwnerElementKey(BoneName, ERigElementType::Bone);
		if (!ExecuteContext.Hierarchy->Find(OwnerElementKey))
		{
			continue;
		}
		if (!BoneMask.IsEmpty() && BoneMask.Find(BoneName) == INDEX_NONE)
		{
			continue;
		}

		FRigComponentKey DynamicsColliderComponentKey = Controller->AddComponent(
			FRigDynamicsColliderComponent::StaticStruct(), ColliderComponentName, OwnerElementKey);

		if (DynamicsColliderComponentKey.IsValid())
		{
			if (FRigDynamicsColliderComponent* Component =
				GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))
			{
				DynamicsColliderComponentKeys.Add(DynamicsColliderComponentKey);

				FRigDynamicsShapeCollection& ColliderCollection = Component->Shapes;

				const FKAggregateGeom& AggGeom = SkeletalBodySetup->AggGeom;
				for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
				{
					FTransform TM(BoxElem.Rotation, BoxElem.Center);
					FRigDynamicsShapeBox Box(
						FName(BoxElem.GetName()), TM, FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
					ColliderCollection.Boxes.Add(Box);
				}
				for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
				{
					FTransform TM(SphereElem.Center);
					FRigDynamicsShapeCapsule Capsule(
						FName(SphereElem.GetName()), TM, SphereElem.Radius, 0.0f);
					ColliderCollection.Capsules.Add(Capsule);
				}
				for (const FKSphylElem& SphylElem : AggGeom.SphylElems)
				{
					FTransform TM(SphylElem.Rotation, SphylElem.Center);
					FRigDynamicsShapeCapsule Capsule(
						FName(SphylElem.GetName()), TM, SphylElem.Radius, SphylElem.Length);
					ColliderCollection.Capsules.Add(Capsule);
				}

				if (Solver)
				{
					Solver->Colliders.Add(DynamicsColliderComponentKey);
				}
			}
		}

	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset::GetUpgradeInfo() const
{
	FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset1 NewNode;
	NewNode.DynamicsSolverComponentKey = DynamicsSolverComponentKey;
	NewNode.ColliderComponentName = ColliderComponentName;
	NewNode.PhysicsAsset = PhysicsAsset;

	NewNode.ElementIncludeMask.Reserve(BoneMask.Num());
	for (const FName& BoneName : BoneMask)
	{
		NewNode.ElementIncludeMask.Add(FRigElementKey(BoneName, ERigElementType::Bone));
	}

	return FRigVMStructUpgradeInfo(*this, NewNode);
}

//======================================================================================================================
FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset1_Execute()
{
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset can only be used during Setup"));
		return;
	}

	// The construction event can be re-entered in preview without re-instantiating the unit, so
	// wipe the output array from prior runs before we start appending fresh keys.
	DynamicsColliderComponentKeys.Reset();

	if (!ExecuteContext.Hierarchy)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset: No Hierarchy"));
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	if (!PhysicsAsset)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset needs a valid physics asset"));
		return;
	}

	URigHierarchyController* Controller = Hierarchy.GetController();
	if (!Controller)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("ImportDynamicsCollidersFromPhysicsAsset: No Controller"));
		return;
	}

	FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey);

	FRigHierarchyControllerInstructionBracket InstructionBracket(Controller, ExecuteContext.GetInstructionIndex());

	for (TObjectPtr<USkeletalBodySetup> SkeletalBodySetup : PhysicsAsset->SkeletalBodySetups)
	{
		if (!SkeletalBodySetup)
		{
			continue;
		}

		// Check we can use this based on the owning element
		FName BoneName = SkeletalBodySetup->BoneName;
		const FRigElementKey OwnerElementKey(BoneName, ERigElementType::Bone);
		if (!ExecuteContext.Hierarchy->Find(OwnerElementKey))
		{
			continue;
		}
		if (!ElementIncludeMask.IsEmpty() && ElementIncludeMask.Find(OwnerElementKey) == INDEX_NONE)
		{
			continue;
		}
		if (ElementExcludeMask.Find(OwnerElementKey) != INDEX_NONE)
		{
			continue;
		}
		if (bExcludeElementsWithParticles && Solver && Solver->Particles.ContainsByPredicate(
			[&OwnerElementKey](const FRigComponentKey& Key) { return Key.ElementKey == OwnerElementKey; }))
		{
			continue;
		}

		FRigComponentKey DynamicsColliderComponentKey = Controller->AddComponent(
			FRigDynamicsColliderComponent::StaticStruct(), ColliderComponentName, OwnerElementKey);

		if (DynamicsColliderComponentKey.IsValid())
		{
			if (FRigDynamicsColliderComponent* Component =
				GetCollider(*ExecuteContext.Hierarchy, DynamicsColliderComponentKey))
			{
				DynamicsColliderComponentKeys.Add(DynamicsColliderComponentKey);

				FRigDynamicsShapeCollection& ColliderCollection = Component->Shapes;

				const FKAggregateGeom& AggGeom = SkeletalBodySetup->AggGeom;
				for (const FKBoxElem& BoxElem : AggGeom.BoxElems)
				{
					FTransform TM(BoxElem.Rotation, BoxElem.Center);
					FRigDynamicsShapeBox Box(
						FName(BoxElem.GetName()), TM, FVector(BoxElem.X, BoxElem.Y, BoxElem.Z));
					ColliderCollection.Boxes.Add(Box);
				}
				for (const FKSphereElem& SphereElem : AggGeom.SphereElems)
				{
					FTransform TM(SphereElem.Center);
					FRigDynamicsShapeCapsule Capsule(
						FName(SphereElem.GetName()), TM, SphereElem.Radius, 0.0f);
					ColliderCollection.Capsules.Add(Capsule);
				}
				for (const FKSphylElem& SphylElem : AggGeom.SphylElems)
				{
					FTransform TM(SphylElem.Rotation, SphylElem.Center);
					FRigDynamicsShapeCapsule Capsule(
						FName(SphylElem.GetName()), TM, SphylElem.Radius, SphylElem.Length);
					ColliderCollection.Capsules.Add(Capsule);
				}

				if (Solver)
				{
					Solver->Colliders.Add(DynamicsColliderComponentKey);
				}
			}
		}

	}
}

//======================================================================================================================
FRigUnit_HierarchyAddDynamicsParticleForceField_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	FRigDynamicsSolverComponent* SolverComponent = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey);
	if (!SolverComponent)
	{
		return;
	}
	if (FRigDynamicsSolver* Solver = SolverComponent->GetDynamicsSolver())
	{
		Solver->AddFieldForcesToParticles(
			*ExecuteContext.Hierarchy, *SolverComponent,
			Space, FieldTransform, Type,
			RadialForce, FieldCurves.RadialMultiplier.GetRichCurveConst(),
			LinearForceDirection, LinearForce, FieldCurves.LinearMultiplier.GetRichCurveConst(),
			RotationalForceAxis, RotationalForce, FieldCurves.RotationalMultiplier.GetRichCurveConst(),
			ExecuteContext.GetDrawInterface(), bDrawDebug, DebugForceScale);
	}
}
