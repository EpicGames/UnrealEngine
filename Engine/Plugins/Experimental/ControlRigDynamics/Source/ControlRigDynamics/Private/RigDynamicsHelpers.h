// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsParticleComponent.h"
#include "RigDynamicsSolverComponent.h"
#include "RigDynamicsColliderComponent.h"
#include "RigDynamicsConfinerComponent.h"
#include "RigDynamicsConstraintComponent.h"
#include "RigDynamicsConeLimitComponent.h"

#include "Rigs/RigHierarchy.h"

//======================================================================================================================
inline FRigDynamicsParticleComponent* GetParticle(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsParticleComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsParticleComponent* GetParticle(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsParticleComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsParticleComponent* GetParticle(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigDynamicsParticleComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
// Mutable variant of the cached lookup. FCachedRigComponent::GetComponent returns a const pointer
// regardless of hierarchy constness; the const_cast is safe here because the caller holds a
// non-const URigHierarchy& and the underlying component lives in that hierarchy.
inline FRigDynamicsParticleComponent* GetParticle(URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return const_cast<FRigDynamicsParticleComponent*>(
		Cast<FRigDynamicsParticleComponent>(CachedComponent.GetComponent(&Hierarchy)));
}

//======================================================================================================================
inline FRigDynamicsSolverComponent* GetSolver(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsSolverComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsSolverComponent* GetSolver(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsSolverComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsSolverComponent* GetSolver(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigDynamicsSolverComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
inline FRigDynamicsColliderComponent* GetCollider(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsColliderComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsColliderComponent* GetCollider(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsColliderComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsColliderComponent* GetCollider(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigDynamicsColliderComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
inline FRigDynamicsConfinerComponent* GetConfiner(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConfinerComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsConfinerComponent* GetConfiner(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConfinerComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsConfinerComponent* GetConfiner(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedComponent)
{
	return Cast<FRigDynamicsConfinerComponent>(CachedComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
inline FRigDynamicsConstraintComponent* GetConstraint(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConstraintComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsConstraintComponent* GetConstraint(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConstraintComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsConstraintComponent* GetConstraint(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	return Cast<FRigDynamicsConstraintComponent>(CachedRigComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
inline FRigDynamicsConeLimitComponent* GetConeLimit(URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConeLimitComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
inline const FRigDynamicsConeLimitComponent* GetConeLimit(const URigHierarchy& Hierarchy, const FRigComponentKey& Key)
{
	return Cast<FRigDynamicsConeLimitComponent>(Hierarchy.FindComponent(Key));
}

//======================================================================================================================
// Updates the cache if necessary and then returns the component
inline const FRigDynamicsConeLimitComponent* GetConeLimit(
	const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	return Cast<FRigDynamicsConeLimitComponent>(CachedRigComponent.GetComponent(&Hierarchy));
}

//======================================================================================================================
// Gets the transform from the element that owns this component. 
inline FTransform GetGlobalTransform(const URigHierarchy& Hierarchy, const FRigBaseComponent& Component)
{
	// Since all components are owned, they must have an element
	check(Component.GetElement());
	return Hierarchy.GetGlobalTransform(Component.GetElement()->GetIndex());
}

//======================================================================================================================
// Gets the location from the element that owns this component. 
inline FVector GetGlobalLocation(const URigHierarchy& Hierarchy, const FRigBaseComponent& Component)
{
	// Since all components are owned, they must have an element
	check(Component.GetElement());
	return Hierarchy.GetGlobalTransform(Component.GetElement()->GetIndex()).GetLocation();
}

//======================================================================================================================
// We assume that the cached rig component is fundamentally valid (i.e. exists), but we do ensure it is cached.
inline int32 GetElementIndex(const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	CachedRigComponent.UpdateCache(&Hierarchy);
	const FRigBaseElement* Element = CachedRigComponent.GetElement();
	// Since all components are owned, they must have an element
	check(Element);
	return Element->GetIndex();
}

//======================================================================================================================
inline FTransform GetGlobalTransform(const URigHierarchy& Hierarchy, FCachedRigComponent& CachedRigComponent)
{
	const int32 Index = GetElementIndex(Hierarchy, CachedRigComponent);
	return Hierarchy.GetGlobalTransform(Index);
}