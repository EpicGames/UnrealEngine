// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
struct FRigDynamicsObjectVersion
{
	enum Type
	{
		FirstVersion,
		ParticleExtraDamping,
		DynamicsTargetMode,
		GravityMultiplier,
		PlaneDefinition,
		PlaneExtents,
		Constraints,
		HelperStructs,
		BonePositionAndOrientationSetting,
		NoCollisionColliders,
		CollisionParticles,
		TargetModeFloat,
		SimulationSpace,
		ConeLimits,
		AngleLimit,
		ResetDetection,
		ParticleValueDisplay,
		ConstraintVisualization,
		AngleLimitVisualization,
		CollideWithColliders,
		ParticleDrag,
		DragSettings,
		Confiners,
		RemoveSolverLevelColliders,
		ConeLimitVisualization,
		TeleportFlags,
		ResetFlags,
		SimulationSpaceRegrouping,
		InertialForcePerTermAmounts,
		SimulationAdditionalDrag,
		EvaluationIntervalReset,
		AccelerationModeAndDamping,

		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FRigDynamicsObjectVersion() {}
};

#undef UE_API
