// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsParticleComponent.h"

#include "RigDynamicsObjectVersion.h"

//======================================================================================================================
void FRigDynamicsParticleComponent::Serialize(FArchive& Ar)
{
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::HelperStructs)
	{
		Ar << ParticleProperties;
	}
	else
	{
		Ar << ParticleProperties.Radius;
		Ar << ParticleProperties.Mass;
		Ar << ParticleProperties.MovementType;
		if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::GravityMultiplier)
		{
			Ar << ParticleProperties.GravityMultiplier;
		}
		Ar << ParticleProperties.Strength;
		Ar << ParticleProperties.DampingRatio;
		if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::ParticleExtraDamping)
		{
			Ar << ParticleProperties.ExtraDamping;
		}
		Ar << ParticleProperties.TargetVelocityInfluence;
		if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::DynamicsTargetMode)
		{
			// Ignore old data (we have no legacy assets)
			uint8 LegacyTargetMode = 1;
			Ar << LegacyTargetMode;
		}
	}
}

//======================================================================================================================
void FRigDynamicsParticleComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);
	FRigBaseComponent::Save(Ar);
	Serialize(Ar);
}

//======================================================================================================================
void FRigDynamicsParticleComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Serialize(Ar);
}
