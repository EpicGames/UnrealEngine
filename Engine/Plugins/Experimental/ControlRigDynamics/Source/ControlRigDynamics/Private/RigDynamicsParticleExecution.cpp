// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsParticleExecution.h"

#include "RigDynamicsHelpers.h"

#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsParticleExecution)

//======================================================================================================================
// Macros to reduce boilerplate for the Get/Set particle property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

#define IMPLEMENT_SET_DYNAMICS_PARTICLE(PropName)                                \
FRigUnit_HierarchySetDynamicsParticle##PropName##_Execute()                      \
{                                                                                \
	if (!ExecuteContext.Hierarchy) { return; }                                   \
	if (FRigDynamicsParticleComponent* Component =                               \
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))    \
	{                                                                            \
		Component->ParticleProperties.PropName = PropName;                       \
	}                                                                            \
}

#define IMPLEMENT_GET_DYNAMICS_PARTICLE(PropName)                                \
FRigUnit_HierarchyGetDynamicsParticle##PropName##_Execute()                      \
{                                                                                \
	if (!ExecuteContext.Hierarchy) { return; }                                   \
	if (const FRigDynamicsParticleComponent* Component =                         \
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))    \
	{                                                                            \
		PropName = Component->ParticleProperties.PropName;                       \
	}                                                                            \
}

//======================================================================================================================
FRigUnit_SpawnDynamicsParticle_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsParticle can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsParticleComponentKey = Controller->AddComponent(
			FRigDynamicsParticleComponent::StaticStruct(), ParticleComponentName, Owner);
		if (DynamicsParticleComponentKey.IsValid())
		{
			if (FRigDynamicsParticleComponent* Component =
				GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
			{
				Component->ParticleProperties = ParticleProperties;
			}

			if (FRigDynamicsSolverComponent* Solver = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Solver->Particles.Add(DynamicsParticleComponentKey);
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsParticleExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey) != nullptr;
}

//======================================================================================================================
FRigUnit_HierarchyDisableDynamicsCollisionWithCollider_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("DisableDynamicsCollisionWithCollider can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent = 
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.NoCollisionColliders.AddUnique(DynamicsColliderComponentKey);
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("DisableDynamicsCollisionWithCollider Unable to find particle"));
	}
}

//======================================================================================================================
FRigUnit_HierarchyAllowDynamicsCollisionWithCollider_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("AllowDynamicsCollisionWithCollider can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.NoCollisionColliders.Remove(DynamicsColliderComponentKey);
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("AllowDynamicsCollisionWithCollider Unable to find particle"));
	}
}

//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleNoCollisionColliders_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(
			TEXT("SetDynamicsParticleNoCollisionColliders can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.NoCollisionColliders = NoCollisionColliders;
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("SetDynamicsParticleNoCollisionColliders Unable to find particle"));
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleNoCollisionColliders_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		NoCollisionColliders = ParticleComponent->ParticleProperties.NoCollisionColliders;
	}
}

//======================================================================================================================
FRigUnit_HierarchyEnableDynamicsCollisionWithParticle_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("EnableDynamicsCollisionWithParticle can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.CollisionParticles.AddUnique(OtherDynamicsParticleComponentKey);
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("EnableDynamicsCollisionWithParticle Unable to find particle"));
	}
}

//======================================================================================================================
FRigUnit_HierarchyDisableDynamicsCollisionWithParticle_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("DisableDynamicsCollisionWithParticle can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.CollisionParticles.Remove(OtherDynamicsParticleComponentKey);
	}

	if (FRigDynamicsParticleComponent* OtherParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, OtherDynamicsParticleComponentKey))
	{
		OtherParticleComponent->ParticleProperties.CollisionParticles.Remove(DynamicsParticleComponentKey);
	}
}

//======================================================================================================================
FRigUnit_HierarchyEnableDynamicsConfinementWithConfiner_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("EnableDynamicsConfinementWithConfiner can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.Confiners.AddUnique(DynamicsConfinerComponentKey);
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("EnableDynamicsConfinementWithConfiner Unable to find particle"));
	}
}

//======================================================================================================================
FRigUnit_HierarchyDisableDynamicsConfinementWithConfiner_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("DisableDynamicsConfinementWithConfiner can only be used during Setup"));
		return;
	}

	if (FRigDynamicsParticleComponent* ParticleComponent =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		ParticleComponent->ParticleProperties.Confiners.Remove(DynamicsConfinerComponentKey);
	}
	else
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_WARNING(TEXT("DisableDynamicsConfinementWithConfiner Unable to find particle"));
	}
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(Radius)
IMPLEMENT_GET_DYNAMICS_PARTICLE(Radius)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(Mass)
IMPLEMENT_GET_DYNAMICS_PARTICLE(Mass)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(MovementType)
IMPLEMENT_GET_DYNAMICS_PARTICLE(MovementType)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(GravityMultiplier)
IMPLEMENT_GET_DYNAMICS_PARTICLE(GravityMultiplier)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(Strength)
IMPLEMENT_GET_DYNAMICS_PARTICLE(Strength)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(DampingRatio)
IMPLEMENT_GET_DYNAMICS_PARTICLE(DampingRatio)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(ExtraDamping)
IMPLEMENT_GET_DYNAMICS_PARTICLE(ExtraDamping)
//======================================================================================================================
// Set Drag puts the particle into drag mode: the stored Damping field holds the drag coefficient and
// bScaleDampingByInverseMass is forced true so the solver computes rate = Damping * InvMass.
//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleDrag_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->ParticleProperties.Damping = Drag;
		Component->ParticleProperties.bScaleDampingByInverseMass = true;
	}
}

//======================================================================================================================
// Get Drag returns the drag-mode interpretation of the particle's current state: in drag mode the
// stored Damping already is the drag coefficient; in damping mode the stored Damping is the rate
// directly, so we multiply by Mass to give an equivalent drag coefficient.
//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleDrag_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Drag = Component->ParticleProperties.bScaleDampingByInverseMass
			? Component->ParticleProperties.Damping
			: Component->ParticleProperties.Damping * Component->ParticleProperties.Mass;
	}
}

//======================================================================================================================
// Set Damping puts the particle into damping mode: the stored Damping field holds the rate directly
// and bScaleDampingByInverseMass is forced false so the solver computes rate = Damping.
//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleDamping_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->ParticleProperties.Damping = Damping;
		Component->ParticleProperties.bScaleDampingByInverseMass = false;
	}
}

//======================================================================================================================
// Get Damping returns the damping-mode interpretation: in damping mode the stored Damping is the
// rate directly; in drag mode the stored Damping is the drag coefficient, so we divide by Mass to
// recover the effective rate.
//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleDamping_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		const float Mass = FMath::Max(Component->ParticleProperties.Mass, KINDA_SMALL_NUMBER);
		Damping = Component->ParticleProperties.bScaleDampingByInverseMass
			? Component->ParticleProperties.Damping / Mass
			: Component->ParticleProperties.Damping;
	}
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(TargetVelocityInfluence)
IMPLEMENT_GET_DYNAMICS_PARTICLE(TargetVelocityInfluence)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(TargetMode)
IMPLEMENT_GET_DYNAMICS_PARTICLE(TargetMode)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(AngleLimit)
IMPLEMENT_GET_DYNAMICS_PARTICLE(AngleLimit)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_PARTICLE(AngleLimitStrength)
IMPLEMENT_GET_DYNAMICS_PARTICLE(AngleLimitStrength)

//======================================================================================================================
// Written out rather than using the Set/Get macros because the struct name (CollideWithColliders)
// and the property/pin name (bCollideWithColliders) differ
//======================================================================================================================

//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleCollideWithColliders_Execute()
{
	if (!ExecuteContext.Hierarchy) { return; }
	if (FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->ParticleProperties.bCollideWithColliders = bCollideWithColliders;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleCollideWithColliders_Execute()
{
	if (!ExecuteContext.Hierarchy) { return; }
	if (const FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		bCollideWithColliders = Component->ParticleProperties.bCollideWithColliders;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleAccelerationMode_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->ParticleProperties.bAccelerationMode = bAccelerationMode;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleAccelerationMode_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		bAccelerationMode = Component->ParticleProperties.bAccelerationMode;
	}
}

//======================================================================================================================
FRigUnit_HierarchySetDynamicsParticleScaleDampingByInverseMass_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->ParticleProperties.bScaleDampingByInverseMass = bScaleDampingByInverseMass;
	}
}

//======================================================================================================================
FRigUnit_HierarchyGetDynamicsParticleScaleDampingByInverseMass_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsParticleComponent* Component =
		GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		bScaleDampingByInverseMass = Component->ParticleProperties.bScaleDampingByInverseMass;
	}
}

//======================================================================================================================
FRigUnit_HierarchyAddDynamicsParticleForce_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (FRigDynamicsParticleComponent* Component = GetParticle(*ExecuteContext.Hierarchy, DynamicsParticleComponentKey))
	{
		Component->PendingForces.Emplace(Force, Space, Type);
	}
}
