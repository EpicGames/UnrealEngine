// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsSimulationSpace.h"

#include "PhysicsControlHelpers.h"

//======================================================================================================================
// TODO We need support for double precision Perlin inputs - for now duplicate.
namespace FPerlinHelpers
{
	// random permutation of 256 numbers, repeated 2x
	static const int32 Permutation[512] = {
		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 
		222, 2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 
		117, 251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 
		173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 
		94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 
		69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235,

		63, 9, 212, 205, 31, 128, 72, 59, 137, 203, 195, 170, 181, 115, 165, 40, 116, 139, 175, 225, 132, 99, 222, 
		2, 41, 15, 197, 93, 169, 90, 228, 43, 221, 38, 206, 204, 73, 17, 97, 10, 96, 47, 32, 138, 136, 30, 219,
		78, 224, 13, 193, 88, 134, 211, 7, 112, 176, 19, 106, 83, 75, 217, 85, 0, 98, 140, 229, 80, 118, 151, 117, 
		251, 103, 242, 81, 238, 172, 82, 110, 4, 227, 77, 243, 46, 12, 189, 34, 188, 200, 161, 68, 76, 171, 194,
		57, 48, 247, 233, 51, 105, 5, 23, 42, 50, 216, 45, 239, 148, 249, 84, 70, 125, 108, 241, 62, 66, 64, 240, 
		173, 185, 250, 49, 6, 37, 26, 21, 244, 60, 223, 255, 16, 145, 27, 109, 58, 102, 142, 253, 120, 149, 160,
		124, 156, 79, 186, 135, 127, 14, 121, 22, 65, 54, 153, 91, 213, 174, 24, 252, 131, 192, 190, 202, 208, 35, 
		94, 231, 56, 95, 183, 163, 111, 147, 25, 67, 36, 92, 236, 71, 166, 1, 187, 100, 130, 143, 237, 178, 158,
		104, 184, 159, 177, 52, 214, 230, 119, 87, 114, 201, 179, 198, 3, 248, 182, 39, 11, 152, 196, 113, 20, 232, 
		69, 141, 207, 234, 53, 86, 180, 226, 74, 150, 218, 29, 133, 8, 44, 123, 28, 146, 89, 101, 154, 220, 126,
		155, 122, 210, 168, 254, 162, 129, 33, 18, 209, 61, 191, 199, 157, 245, 55, 164, 167, 215, 246, 144, 107, 235
	};

	//==================================================================================================================
	FORCEINLINE float Grad1(int32 Hash, double X)
	{
		static const double Grad1Scales[16] = {
			-8 / 8., -7 / 8., -6 / 8., -5 / 8., -4 / 8., -3 / 8., -2 / 8., -1 / 8.,
			1 / 8., 2 / 8., 3 / 8., 4 / 8., 5 / 8., 6 / 8., 7 / 8., 8 / 8. };
		return Grad1Scales[Hash & 15] * X;
	}

	//==================================================================================================================
	FORCEINLINE double SmoothCurve(double X)
	{
		return X * X * X * (X * (X * 6.0 - 15.0) + 10.0);
	}

	//==================================================================================================================
	FORCEINLINE float PerlinNoise1D(double X)
	{
		const double Xfl = FMath::FloorToDouble(X);
		const int64 Xi = (int64)(Xfl) & 255;
		X -= Xfl;
		const double Xm1 = X - 1.0;

		const int32 A = Permutation[Xi];
		const int32 B = Permutation[Xi + 1];

		const double U = SmoothCurve(X);

		// 2.0 factor to ensure (-1, 1) range
		return 2.0f * FMath::Lerp(Grad1(A, X), Grad1(B, Xm1), U);
	}
}

//======================================================================================================================
// Returns true if the movement between the stored SimulationSpaceTM and the incoming TM (and the
// just-computed accelerations) crosses any of the teleport thresholds. Each per-channel bool gates
// its own check independently; this function ignores motion-shaping (clamps / vertical scale) and
// inertial-force settings entirely.
static bool IsSimulationSpaceTeleport(
	const FRigDynamicsTeleportDetectionSettings& Detect,
	const FTransform&                            OldSimulationSpaceTM,
	const FTransform&                            NewSimulationSpaceTM,
	const FVector&                               NewLinearAcceleration,
	const FVector&                               NewAngularAcceleration)
{
	if (Detect.bFromPositionChange)
	{
		const double DeltaPosSq =
			(NewSimulationSpaceTM.GetTranslation() - OldSimulationSpaceTM.GetTranslation()).SizeSquared();
		const double Threshold = Detect.PositionChangeThreshold;
		if (DeltaPosSq > Threshold * Threshold)
		{
			return true;
		}
	}

	if (Detect.bFromOrientationChange)
	{
		const float DeltaAngleDeg = FMath::RadiansToDegrees(
			NewSimulationSpaceTM.GetRotation().AngularDistance(OldSimulationSpaceTM.GetRotation()));
		if (DeltaAngleDeg > Detect.OrientationChangeThreshold)
		{
			return true;
		}
	}

	if (Detect.bFromLinearAcceleration)
	{
		const double Threshold = Detect.LinearAccelerationThreshold;
		if (NewLinearAcceleration.SizeSquared() > Threshold * Threshold)
		{
			return true;
		}
	}

	if (Detect.bFromAngularAcceleration)
	{
		// Threshold is authored in deg/s/s; stored angular acceleration is rad/s/s
		const double ThresholdRad = FMath::DegreesToRadians(Detect.AngularAccelerationThreshold);
		if (NewAngularAcceleration.SizeSquared() > ThresholdRad * ThresholdRad)
		{
			return true;
		}
	}

	return false;
}

//======================================================================================================================
void FRigDynamicsSimulationSpaceState::Update(
	const FRigDynamicsTeleportDetectionSettings& TeleportDetection,
	const FTransform& InComponentTM,
	const FTransform& InSimulationSpaceTM,
	float InDeltaTime)
{
	if (InDeltaTime > 0.0f)
	{
		FVector NewLinearVelocity = UE::PhysicsControl::CalculateLinearVelocity(
			SimulationSpaceTM.GetTranslation(), InSimulationSpaceTM.GetTranslation(), InDeltaTime);
		FVector NewAngularVelocity = UE::PhysicsControl::CalculateAngularVelocity(
			SimulationSpaceTM.GetRotation(), InSimulationSpaceTM.GetRotation(), InDeltaTime);

		FVector NewLinearAcceleration = FVector::ZeroVector;
		FVector NewAngularAcceleration = FVector::ZeroVector;
		if (UpdatesSinceReset > 0)
		{
			NewLinearAcceleration = (NewLinearVelocity - LinearVelocity) / InDeltaTime;
			NewAngularAcceleration = (NewAngularVelocity - AngularVelocity) / InDeltaTime;
		}

		const bool bTeleport = IsSimulationSpaceTeleport(
			TeleportDetection, SimulationSpaceTM, InSimulationSpaceTM, NewLinearAcceleration, NewAngularAcceleration);
		bTeleportDetectedInLastUpdate = bTeleport;

		if (bTeleport)
		{
			// Suppress bogus motion this frame. Zeroing velocity means CalculateMotion injects no
			// air/ether drag / Coriolis / centrifugal for this frame. Resetting UpdatesSinceReset to 0
			// means the next Update's acceleration will be forced to zero, and CalculateMotion will
			// mark bAccelerationsValid = false (needs >= 2) for one more frame.
			LinearVelocity = FVector::ZeroVector;
			AngularVelocity = FVector::ZeroVector;
			LinearAcceleration = FVector::ZeroVector;
			AngularAcceleration = FVector::ZeroVector;
			UpdatesSinceReset = 0;
		}
		else
		{
			LinearVelocity = NewLinearVelocity;
			AngularVelocity = NewAngularVelocity;
			LinearAcceleration = NewLinearAcceleration;
			AngularAcceleration = NewAngularAcceleration;
			++UpdatesSinceReset;
		}

		PrevPrevSimulationSpaceTM = PrevSimulationSpaceTM;
		PrevSimulationSpaceTM = SimulationSpaceTM;

		PrevDeltaTime = DeltaTime;
		DeltaTime = InDeltaTime;
	}

	// If delta time is zero just update the transform and don't worry too much about velocities being off.
	SimulationSpaceTM = InSimulationSpaceTM;
	ComponentTM = InComponentTM;
	RecomputeCompositeTransforms();
}

//======================================================================================================================
void FRigDynamicsSimulationSpaceState::Reset(
	const FTransform& InComponentTM, const FTransform& InSimulationSpaceTM)
{
	*this = FRigDynamicsSimulationSpaceState();
	SimulationSpaceTM = InSimulationSpaceTM;
	ComponentTM = InComponentTM;
	RecomputeCompositeTransforms();
}

//======================================================================================================================
void FRigDynamicsSimulationSpaceState::RecomputeCompositeTransforms()
{
	// ComponentToSimSpace: component-space pos -> world (via ComponentTM) -> sim-space (via SimTM^-1)
	// In UE composition (A*B means "apply A then B"): ComponentTM * SimulationSpaceTM.Inverse()
	ComponentToSimSpaceTM = ComponentTM * SimulationSpaceTM.Inverse();
	ComponentToSimSpaceTM.SetScale3D(FVector::OneVector);

	SimToComponentSpaceTM = SimulationSpaceTM * ComponentTM.Inverse();
	SimToComponentSpaceTM.SetScale3D(FVector::OneVector);
}

//======================================================================================================================
RigParticleSimulation::FSimulationSpaceMotion FRigDynamicsSimulationSpaceState::CalculateMotion(
	const FRigDynamicsSimulationSpaceMotion& SpaceMotion,
	const double                             AbsoluteTime) const
{
	using RigParticleSimulation::FSimVector;

	const FRigDynamicsSimulationDragSettings& Drag = SpaceMotion.Drag;

	// Sim-space motion in world coordinates, before transforming to sim space
	FVector WorldLinVel = LinearVelocity;
	FVector WorldAngVel = AngularVelocity;
	FVector WorldLinAccel = LinearAcceleration;
	FVector WorldAngAccel = AngularAcceleration;

	WorldLinVel.Z *= SpaceMotion.VerticalMotionScale;
	WorldLinAccel.Z *= SpaceMotion.VerticalMotionScale;

	if (SpaceMotion.bClampLinearVelocity)
	{
		WorldLinVel = WorldLinVel.GetClampedToMaxSize(SpaceMotion.MaxLinearVelocity);
	}
	if (SpaceMotion.bClampAngularVelocity)
	{
		// Stored angular velocity is rad/s; MaxAngularVelocity is authored in deg/s.
		WorldAngVel = WorldAngVel.GetClampedToMaxSize(FMath::DegreesToRadians(SpaceMotion.MaxAngularVelocity));
	}
	if (SpaceMotion.bClampLinearAcceleration)
	{
		WorldLinAccel = WorldLinAccel.GetClampedToMaxSize(SpaceMotion.MaxLinearAcceleration);
	}
	if (SpaceMotion.bClampAngularAcceleration)
	{
		// Stored angular acceleration is rad/s/s; MaxAngularAcceleration is authored in deg/s/s.
		WorldAngAccel = WorldAngAccel.GetClampedToMaxSize(FMath::DegreesToRadians(SpaceMotion.MaxAngularAcceleration));
	}

	// External air/ether velocity: user-authored wind / vortex / turbulence. Kept separate from the
	// sim-space motion above so the drag multipliers only scale the sim-space contribution.
	FVector WorldExtLinVel = Drag.ExternalLinearVelocity;
	FVector WorldExtAngVel = FMath::DegreesToRadians(Drag.ExternalAngularVelocity);
	if (!Drag.ExternalTurbulenceVelocity.IsNearlyZero())
	{
		const FVector T(
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime),
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime + 10.0),
			FPerlinHelpers::PerlinNoise1D(AbsoluteTime + 20.0));
		WorldExtLinVel += T * Drag.ExternalTurbulenceVelocity;
	}

	// Transform everything into simulation space
	RigParticleSimulation::FSimulationSpaceMotion Result;
	Result.LinearVelocity = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldLinVel));
	Result.AngularVelocity = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldAngVel));
	Result.LinearAcceleration = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldLinAccel));
	Result.AngularAcceleration = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldAngAccel));
	Result.ExternalLinearVelocity = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldExtLinVel));
	Result.ExternalAngularVelocity = FSimVector(SimulationSpaceTM.InverseTransformVectorNoScale(WorldExtAngVel));

	Result.InertialForceAmount = SpaceMotion.InertialForces.Amount;
	Result.LinearEulerAmount   = SpaceMotion.InertialForces.LinearEulerAmount;
	Result.AngularEulerAmount  = SpaceMotion.InertialForces.AngularEulerAmount;
	Result.CentrifugalAmount   = SpaceMotion.InertialForces.CentrifugalAmount;
	Result.CoriolisAmount      = SpaceMotion.InertialForces.CoriolisAmount;
	Result.LinearDragMultiplier = FMath::Clamp(Drag.LinearDragMultiplier, 0.0f, 1.0f);
	Result.AngularDragMultiplier = FMath::Clamp(Drag.AngularDragMultiplier, 0.0f, 1.0f);
	Result.AdditionalDamping = FMath::Max(Drag.AdditionalDamping, 0.0f);
	Result.bAccelerationsValid = UpdatesSinceReset >= 2;

	return Result;
}
