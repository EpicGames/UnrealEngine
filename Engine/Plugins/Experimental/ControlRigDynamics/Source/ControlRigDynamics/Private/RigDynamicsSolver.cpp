// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsSolver.h"

#include "RigParticleSimulation.h"

#include "Rigs/RigHierarchy.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "RigVMCore/RigVMExecuteContext.h"

#include "Curves/RichCurve.h"

DEFINE_LOG_CATEGORY(LogRigDynamics);

TAutoConsoleVariable<float> CVarControlRigDynamicsMaxTimeStepOverride(
	TEXT("ControlRig.Dynamics.MaxTimeStepOverride"), -1,
	TEXT("-1 disables the override, so the max timestep authored in the simulation settings will be used. A +ve value is used to specify the maximum timestep."));

TAutoConsoleVariable<int> CVarControlRigDynamicsMaxNumStepsOverride(
	TEXT("ControlRig.Dynamics.MaxNumStepsOverride"), -1,
	TEXT("-1 disables the override, so the max number of steps authored in the simulation settings will be used. A +ve value is used to specify the maximum number of timesteps."));

//======================================================================================================================
FRigDynamicsSolver::FRigDynamicsSolver(const FName InOwnerName) : OwnerName(InOwnerName)
{
}

//======================================================================================================================
int32 FRigDynamicsSolver::GetParticleIndexSlow(const FRigComponentKey& ComponentKey) const
{
	for (int32 Index = 0; Index < ParticleOwnerComponents.Num(); ++Index)
	{
		if (ParticleOwnerComponents[Index].GetComponentKey() == ComponentKey)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

//======================================================================================================================
int32 FRigDynamicsSolver::GetParticleIndexSlow(const FRigBaseElement& Element, const URigHierarchy& Hierarchy)
{
	for (int32 Index = 0; Index < ParticleOwnerComponents.Num(); ++Index)
	{
		if (ParticleOwnerComponents[Index].GetElement(&Hierarchy) == &Element)
		{
			return Index;
		}
	}
	return INDEX_NONE;
}

//======================================================================================================================
int32 FRigDynamicsSolver::GetParentParticleIndexSlow(
	const FRigBaseElement& Element, const URigHierarchy& Hierarchy)
{
	const FRigBaseElement* ParentElement = Hierarchy.GetFirstParent(&Element);
	while (ParentElement)
	{
		int32 Index = GetParticleIndexSlow(*ParentElement, Hierarchy);
		if (Index != INDEX_NONE)
		{
			return Index;
		}
		ParentElement = Hierarchy.GetFirstParent(ParentElement);
	}
	return INDEX_NONE;
}

//======================================================================================================================
int32 FRigDynamicsSolver::GetParentParticleIndexSlow(int32 ParticleIndex, const URigHierarchy& Hierarchy)
{
	if (ParticleOwnerComponents.IsValidIndex(ParticleIndex))
	{
		if (const FRigBaseElement* Element = ParticleOwnerComponents[ParticleIndex].GetElement(&Hierarchy))
		{
			return GetParentParticleIndexSlow(*Element, Hierarchy);
		}
	}
	return INDEX_NONE;
}

//======================================================================================================================
void FRigDynamicsSolver::Instantiate(
	const FRigVMExecuteContext&        ExecuteContext,
	const URigHierarchy&               Hierarchy,
	const FRigDynamicsSolverComponent& SolverComponent)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamicsSolver_Instantiate);

	if (!bNeedToInstantiate)
	{
		return;
	}

	// Build a simulation space state with just the transforms set (DeltaTime=0 skips velocity
	// calculation). This is needed for coordinate conversions during instantiation.
	FRigDynamicsSimulationSpaceState SpaceState;
	UpdateSimulationSpaceStateTransforms(ExecuteContext, Hierarchy, SolverComponent, 0.0f, SpaceState);

	ParticleTargetTMs.Reset();
	ParticleOwnerComponents.Reset();
	HardDistanceConstraintOwnerComponents.Reset();
	SoftDistanceConstraintOwnerComponents.Reset();
	ColliderOwnerComponents.Reset();
	ConfinerOwnerComponents.Reset();
	ConeLimitOwnerComponents.Reset();

	SimulationState = RigParticleSimulation::FSimulationState();

	InstantiateColliders(Hierarchy, SolverComponent, SpaceState);
	InstantiateConfiners(Hierarchy, SolverComponent, SpaceState);
	InstantiateParticles(Hierarchy, SolverComponent, SpaceState);
	ComputeParticleParents(Hierarchy);
	SortParticles();
	SetParticlesNoCollision(Hierarchy);
	InstantiateSkeletalConstraints(Hierarchy);
	InstantiateDistanceConstraints(Hierarchy, SolverComponent);
	InstantiateConeLimits(Hierarchy, SolverComponent);
	SortConeLimits();

	bNeedToInstantiate = false;
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateColliders(
	const URigHierarchy&                    Hierarchy,
	const FRigDynamicsSolverComponent&      SolverComponent,
	const FRigDynamicsSimulationSpaceState& SpaceState)
{
	// Bone-attached colliders. Added first so they're there ready to set no-collision when we
	// add the particles.
	for (const FRigComponentKey& ColliderComponentKey : SolverComponent.Colliders)
	{
		if (const FRigDynamicsColliderComponent* ColliderComponent = GetCollider(Hierarchy, ColliderComponentKey))
		{
			RigParticleSimulation::FShapeCollection& Collider = SimulationState.Colliders.AddDefaulted_GetRef();
			FCachedRigComponent& CachedOwnerComponent = ColliderOwnerComponents.Add_GetRef(
				FCachedRigComponent(ColliderComponentKey, &Hierarchy, true));

			Collider.TargetTM = SpaceState.ConvertComponentSpaceTransformToSimSpace(
				GetGlobalTransform(Hierarchy, CachedOwnerComponent));
			Collider.PreviousTM = Collider.TargetTM;

			for (const FRigDynamicsShapeBox& Box : ColliderComponent->Shapes.Boxes)
			{
				RigParticleSimulation::FBoxShape& BoxShape = Collider.BoxShapes.AddDefaulted_GetRef();
				BoxShape.TM = FSimTransform(Box.TM);
				BoxShape.Extents = FSimVector(Box.Extents);
			}
			for (const FRigDynamicsShapeCapsule& Capsule : ColliderComponent->Shapes.Capsules)
			{
				RigParticleSimulation::FCapsuleShape& CapsuleShape =
					Collider.CapsuleShapes.AddDefaulted_GetRef();
				CapsuleShape.TM = FSimTransform(Capsule.TM);
				CapsuleShape.Length = Capsule.Length;
				CapsuleShape.Radius = Capsule.Radius;
			}
			for (const FRigDynamicsShapePlane& Plane : ColliderComponent->Shapes.Planes)
			{
				RigParticleSimulation::FPlaneShape& PlaneShape =
					Collider.PlaneShapes.AddDefaulted_GetRef();
				PlaneShape.TM = FSimTransform(Plane.TM);
				PlaneShape.Extents = FSimVector2D(Plane.Extents);
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateConfiners(
	const URigHierarchy&                    Hierarchy,
	const FRigDynamicsSolverComponent&      SolverComponent,
	const FRigDynamicsSimulationSpaceState& SpaceState)
{
	for (const FRigComponentKey& ConfinerComponentKey : SolverComponent.Confiners)
	{
		if (const FRigDynamicsConfinerComponent* ConfinerComponent = GetConfiner(Hierarchy, ConfinerComponentKey))
		{
			RigParticleSimulation::FShapeCollection& Confiner = SimulationState.Confiners.AddDefaulted_GetRef();
			SimulationState.ConfinerStrengths.Add(ConfinerComponent->Strength);
			FCachedRigComponent& CachedOwnerComponent = ConfinerOwnerComponents.Add_GetRef(
				FCachedRigComponent(ConfinerComponentKey, &Hierarchy, true));

			Confiner.TargetTM = SpaceState.ConvertComponentSpaceTransformToSimSpace(
				GetGlobalTransform(Hierarchy, CachedOwnerComponent));
			Confiner.PreviousTM = Confiner.TargetTM;

			for (const FRigDynamicsShapeBox& Box : ConfinerComponent->Shapes.Boxes)
			{
				RigParticleSimulation::FBoxShape& BoxShape = Confiner.BoxShapes.AddDefaulted_GetRef();
				BoxShape.TM = FSimTransform(Box.TM);
				BoxShape.Extents = FSimVector(Box.Extents);
			}
			for (const FRigDynamicsShapeCapsule& Capsule : ConfinerComponent->Shapes.Capsules)
			{
				RigParticleSimulation::FCapsuleShape& CapsuleShape =
					Confiner.CapsuleShapes.AddDefaulted_GetRef();
				CapsuleShape.TM = FSimTransform(Capsule.TM);
				CapsuleShape.Length = Capsule.Length;
				CapsuleShape.Radius = Capsule.Radius;
			}
			for (const FRigDynamicsShapePlane& Plane : ConfinerComponent->Shapes.Planes)
			{
				RigParticleSimulation::FPlaneShape& PlaneShape =
					Confiner.PlaneShapes.AddDefaulted_GetRef();
				PlaneShape.TM = FSimTransform(Plane.TM);
				PlaneShape.Extents = FSimVector2D(Plane.Extents);
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateParticles(
	const URigHierarchy&                    Hierarchy,
	const FRigDynamicsSolverComponent&      SolverComponent,
	const FRigDynamicsSimulationSpaceState& SpaceState)
{
	const int32 NumColliders = SimulationState.Colliders.Num();
	const int32 NumColliderOwners = ColliderOwnerComponents.Num();
	const int32 NumConfinerOwners = ConfinerOwnerComponents.Num();

	for (const FRigComponentKey& ParticleComponentKey : SolverComponent.Particles)
	{
		if (const FRigDynamicsParticleComponent* ParticleComponent = GetParticle(Hierarchy, ParticleComponentKey))
		{
			RigParticleSimulation::FParticleInfo& ParticleInfo = SimulationState.ParticleInfos.AddDefaulted_GetRef();
			RigParticleSimulation::FParticle& Particle = SimulationState.Particles.AddDefaulted_GetRef();
			RigParticleSimulation::FParticleNoCollision& ParticleNoCollision =
				SimulationState.ParticleNoCollision.AddDefaulted_GetRef();
			ParticleNoCollision.NoCollisionColliderIndices.Init(false, NumColliders);
			SimulationState.ParticleToParticleCollision.AddDefaulted();
			RigParticleSimulation::FParticleConfinement& ParticleConfinement =
				SimulationState.ParticleConfinement.AddDefaulted_GetRef();
			SimulationState.ParticleTargets.AddDefaulted();
			SimulationState.ParticleColliders.AddDefaulted();

			ParticleOwnerComponents.Add(FCachedRigComponent(ParticleComponentKey, &Hierarchy, true));
			ParticleTargetTMs.Add(GetGlobalTransform(Hierarchy, *ParticleComponent));

			ParticleInfo.TargetPosition = FSimVector(
				SpaceState.ConvertComponentSpacePositionToSimSpace(GetGlobalLocation(Hierarchy, *ParticleComponent)));
			ParticleInfo.PreviousTargetPosition = ParticleInfo.TargetPosition;
			ParticleInfo.GravityMultiplier = ParticleComponent->ParticleProperties.GravityMultiplier;
			ParticleInfo.MovementType = ParticleComponent->ParticleProperties.MovementType;

			Particle.Position = ParticleInfo.TargetPosition;
			Particle.PrevPosition = ParticleInfo.TargetPosition;

			// Set inverse mass here as it can be overridden if the particle needs to be kinematic
			Particle.InvMass =
				ParticleComponent->ParticleProperties.MovementType == ERigParticleSimulationMovementType::Simulated
				? 1.0f / FMath::Max(ParticleComponent->ParticleProperties.Mass, KINDA_SMALL_NUMBER)
				: 0.0f;

			// Target and collision data etc will be set during the updates, so don't do it here

			// Add the no-collision between particles and colliders
			for (const FRigComponentKey& NoCollisionColliderKey :
				ParticleComponent->ParticleProperties.NoCollisionColliders)
			{
				for (int32 ColliderIndex = 0; ColliderIndex != NumColliderOwners; ++ColliderIndex)
				{
					if (ColliderOwnerComponents[ColliderIndex].GetComponentKey() == NoCollisionColliderKey)
					{
						ParticleNoCollision.NoCollisionColliderIndices[ColliderIndex] = true;
					}
				}
			}

			// Resolve this particle's opted-in confiners to solver indices
			for (const FRigComponentKey& ConfinerKey : ParticleComponent->ParticleProperties.Confiners)
			{
				for (int32 ConfinerIndex = 0; ConfinerIndex != NumConfinerOwners; ++ConfinerIndex)
				{
					if (ConfinerOwnerComponents[ConfinerIndex].GetComponentKey() == ConfinerKey)
					{
						ParticleConfinement.ConfinerIndices.AddUnique(ConfinerIndex);
					}
				}
			}

			// Particle-to-particle collision pairs are resolved in SetParticlesNoCollision, after
			// all particles have been added and the sort has happened.
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::ComputeParticleParents(const URigHierarchy& Hierarchy)
{
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
	{
		RigParticleSimulation::FParticleInfo& ParticleInfo = SimulationState.ParticleInfos[ParticleIndex];
		ParticleInfo.ParentParticleIndex = GetParentParticleIndexSlow(ParticleIndex, Hierarchy);

		if (ParticleInfo.ParentParticleIndex == INDEX_NONE)
		{
			// We didn't find a parent - make sure the child is kinematic, as this will be the
			// root-most particle in a chain, and we don't want it to fall off.
			SimulationState.Particles[ParticleIndex].InvMass = 0.0f;
			ParticleInfo.MovementType = ERigParticleSimulationMovementType::Kinematic;
		}
		else
		{
			RigParticleSimulation::FParticleInfo& ParentParticleInfo =
				SimulationState.ParticleInfos[ParticleInfo.ParentParticleIndex];
			++ParentParticleInfo.NumChildren;

			// Check if the parent particle's bone is the direct skeleton parent (no skipped
			// bones). When true, we can avoid a GetGlobalTransform on the child in
			// UpdatePostDynamics.
			const FRigBaseElement* ChildElement = ParticleOwnerComponents[ParticleIndex].GetElement(&Hierarchy);
			const FRigBaseElement* DirectParent = Hierarchy.GetFirstParent(ChildElement);
			const FRigBaseElement* ParentParticleElement =
				ParticleOwnerComponents[ParticleInfo.ParentParticleIndex].GetElement(&Hierarchy);
			ParticleInfo.bParentParticleIsDirectParent = (DirectParent == ParentParticleElement);
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::SortParticles()
{
	const int32 NumParticles = SimulationState.Particles.Num();

	// Sort particles root-to-leaf. Skeletal constraints created later will inherit this order.
	const TArray<int32> SortedOrder = RigParticleSimulation::SortParticlesRootToLeaf(SimulationState);

	TArray<FTransform> TempTMs;              TempTMs.SetNum(NumParticles);
	TArray<FCachedRigComponent> TempOwners;  TempOwners.SetNum(NumParticles);
	for (int32 NewIndex = 0; NewIndex < NumParticles; ++NewIndex)
	{
		TempTMs[NewIndex] = MoveTemp(ParticleTargetTMs[SortedOrder[NewIndex]]);
		TempOwners[NewIndex] = MoveTemp(ParticleOwnerComponents[SortedOrder[NewIndex]]);
	}
	ParticleTargetTMs = MoveTemp(TempTMs);
	ParticleOwnerComponents = MoveTemp(TempOwners);
}

//======================================================================================================================
void FRigDynamicsSolver::SetParticlesNoCollision(const URigHierarchy& Hierarchy)
{
	const int32 NumParticles = SimulationState.Particles.Num();
	const int32 NumColliderOwners = ColliderOwnerComponents.Num();

	for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
	{
		// Automatically disable collision between particles and colliders, if they are on the
		// same element (which is generally a bad idea, as colliders don't track the
		// live/simulated pose)
		RigParticleSimulation::FParticleNoCollision& ParticleNoCollision =
			SimulationState.ParticleNoCollision[ParticleIndex];

		// Don't collide with self (only check component-based colliders which have owner elements)
		{
			const FRigElementKey& OwningElementKey = ParticleOwnerComponents[ParticleIndex].GetElementKey();
			for (int32 ColliderIndex = 0; ColliderIndex != NumColliderOwners; ++ColliderIndex)
			{
				const FRigElementKey& ColliderElementKey = ColliderOwnerComponents[ColliderIndex].GetElementKey();
				if (OwningElementKey == ColliderElementKey)
				{
					ParticleNoCollision.NoCollisionColliderIndices[ColliderIndex] = true;
				}
			}
		}

		// And don't collide with the parent collider either
		const RigParticleSimulation::FParticleInfo& ParticleInfo = SimulationState.ParticleInfos[ParticleIndex];
		int32 ParentParticleIndex = ParticleInfo.ParentParticleIndex;
		if (ParentParticleIndex != INDEX_NONE)
		{
			const FRigElementKey& OwningElementKey = ParticleOwnerComponents[ParentParticleIndex].GetElementKey();
			for (int32 ColliderIndex = 0; ColliderIndex != NumColliderOwners; ++ColliderIndex)
			{
				const FRigElementKey& ColliderElementKey = ColliderOwnerComponents[ColliderIndex].GetElementKey();
				if (OwningElementKey == ColliderElementKey)
				{
					ParticleNoCollision.NoCollisionColliderIndices[ColliderIndex] = true;
				}
			}
		}
	}

	// Resolve particle-to-particle collision pairs. This is done as a second pass so that all
	// particles have been added and can be found by key. Each unique pair is stored only on the
	// lower-indexed particle to avoid solving the same pair twice per iteration. Note that the
	// solve will project both particles at the same time, so the interaction only needs to be
	// stored on one of them.
	for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
	{
		if (const FRigDynamicsParticleComponent* PC = GetParticle(Hierarchy, ParticleOwnerComponents[ParticleIndex]))
		{
			for (const FRigComponentKey& CollisionParticleKey : PC->ParticleProperties.CollisionParticles)
			{
				const int32 OtherIndex = GetParticleIndexSlow(CollisionParticleKey);
				if (OtherIndex != INDEX_NONE && OtherIndex != ParticleIndex)
				{
					const int32 LowerIndex = FMath::Min(ParticleIndex, OtherIndex);
					const int32 UpperIndex = FMath::Max(ParticleIndex, OtherIndex);
					SimulationState.ParticleToParticleCollision[LowerIndex]
						.CollisionParticleIndices.AddUnique(UpperIndex);
				}
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateSkeletalConstraints(const URigHierarchy& Hierarchy)
{
	// One constraint per non-root particle, between the particle and its parent (which may not be
	// the immediate skeleton parent).
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
	{
		const int32 ParentIndex = SimulationState.ParticleInfos[ParticleIndex].ParentParticleIndex;

		if (ParentIndex != INDEX_NONE)
		{
			RigParticleSimulation::FHardDistanceConstraintInfo& ConstraintInfo =
				SimulationState.SkeletalConstraintInfos.AddDefaulted_GetRef();
			RigParticleSimulation::FHardDistanceConstraint& Constraint =
				SimulationState.SkeletalConstraints.AddDefaulted_GetRef();
			// Note that we don't actually need to know the owner (of the child particle). Even if
			// we want to update based on the bone lengths changing, we can get that from the
			// particle positions.

			// Leave the constraint component invalid in ConstraintInfo as we don't have one
			ConstraintInfo.ParentIndex = ParentIndex;
			ConstraintInfo.ChildIndex = ParticleIndex;

			Constraint.TargetDistance = FSimVector::Dist(
				FSimVector(GetGlobalTransform(Hierarchy, ParticleOwnerComponents[ParticleIndex]).GetLocation()),
				FSimVector(GetGlobalTransform(Hierarchy, ParticleOwnerComponents[ParentIndex]).GetLocation()));
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateDistanceConstraints(
	const URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent)
{
	for (const FRigComponentKey& ConstraintComponentKey : SolverComponent.Constraints)
	{
		if (const FRigDynamicsConstraintComponent* ConstraintComponent =
			GetConstraint(Hierarchy, ConstraintComponentKey))
		{
			int32 ParentIndex = GetParticleIndexSlow(ConstraintComponent->ParentComponentKey);
			int32 ChildIndex = GetParticleIndexSlow(ConstraintComponent->ChildComponentKey);

			if (ParentIndex == INDEX_NONE || ChildIndex == INDEX_NONE)
			{
				continue;
			}

			switch (ConstraintComponent->ConstraintType)
			{
			case ERigParticleSimulationConstraintType::Hard:
			{
				RigParticleSimulation::FHardDistanceConstraintInfo& ConstraintInfo =
					SimulationState.HardDistanceConstraintInfos.AddDefaulted_GetRef();
				SimulationState.HardDistanceConstraints.AddDefaulted();
				HardDistanceConstraintOwnerComponents.Add(FCachedRigComponent(ConstraintComponentKey, &Hierarchy, true));

				ConstraintInfo.ParentIndex = ParentIndex;
				ConstraintInfo.ChildIndex = ChildIndex;
				// The rest will be set during the update
			}
			break;
			case ERigParticleSimulationConstraintType::Soft:
			{
				RigParticleSimulation::FSoftDistanceConstraintInfo& ConstraintInfo =
					SimulationState.SoftDistanceConstraintInfos.AddDefaulted_GetRef();
				SimulationState.SoftDistanceConstraints.AddDefaulted();
				SoftDistanceConstraintOwnerComponents.Add(FCachedRigComponent(ConstraintComponentKey, &Hierarchy, true));

				ConstraintInfo.ParentIndex = ParentIndex;
				ConstraintInfo.ChildIndex = ChildIndex;
				// The rest will be set during the update
			}
			break;
			default:
				// Unknown constraint type
				break;
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::InstantiateConeLimits(
	const URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent)
{
	for (const FRigComponentKey& ConeLimitComponentKey : SolverComponent.ConeLimits)
	{
		if (const FRigDynamicsConeLimitComponent* CC = GetConeLimit(Hierarchy, ConeLimitComponentKey))
		{
			int32 GrandparentIndex = GetParticleIndexSlow(CC->GrandparentComponentKey);
			int32 ParentIndex = GetParticleIndexSlow(CC->ParentComponentKey);
			int32 ChildIndex = GetParticleIndexSlow(CC->ChildComponentKey);

			if (GrandparentIndex == INDEX_NONE || ParentIndex == INDEX_NONE || ChildIndex == INDEX_NONE)
			{
				continue;
			}

			RigParticleSimulation::FConeLimitInfo& Info =
				SimulationState.ConeLimitInfos.AddDefaulted_GetRef();
			SimulationState.ConeLimits.AddDefaulted();
			ConeLimitOwnerComponents.Add(FCachedRigComponent(ConeLimitComponentKey, &Hierarchy, true));

			Info.GrandparentIndex = GrandparentIndex;
			Info.ParentIndex = ParentIndex;
			Info.ChildIndex = ChildIndex;
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::SortConeLimits()
{
	// Sort cone limits by child index (root-to-leaf) for better iterative convergence.
	if (SimulationState.ConeLimits.Num() <= 1)
	{
		return;
	}

	const int32 NumConeLimits = SimulationState.ConeLimits.Num();
	TArray<int32> ConeLimitOrder;
	ConeLimitOrder.SetNum(NumConeLimits);
	for (int32 Index = 0; Index < NumConeLimits; ++Index)
	{
		ConeLimitOrder[Index] = Index;
	}
	ConeLimitOrder.Sort([&](int32 A, int32 B)
	{
		return SimulationState.ConeLimitInfos[A].ChildIndex < SimulationState.ConeLimitInfos[B].ChildIndex;
	});

	TArray<RigParticleSimulation::FConeLimitInfo> TempInfos;
	TArray<RigParticleSimulation::FConeLimit> TempLimits;
	TArray<FCachedRigComponent> TempOwners;
	TempInfos.SetNum(NumConeLimits);
	TempLimits.SetNum(NumConeLimits);
	TempOwners.SetNum(NumConeLimits);

	for (int32 Index = 0; Index < NumConeLimits; ++Index)
	{
		TempInfos[Index]  = MoveTemp(SimulationState.ConeLimitInfos[ConeLimitOrder[Index]]);
		TempLimits[Index] = MoveTemp(SimulationState.ConeLimits[ConeLimitOrder[Index]]);
		TempOwners[Index] = MoveTemp(ConeLimitOwnerComponents[ConeLimitOrder[Index]]);
	}
	SimulationState.ConeLimitInfos = MoveTemp(TempInfos);
	SimulationState.ConeLimits = MoveTemp(TempLimits);
	ConeLimitOwnerComponents = MoveTemp(TempOwners);
}

//======================================================================================================================
void FRigDynamicsSolver::UpdatePreDynamics(
	URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent, float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_UpdatePreDynamics);

	// We're an internal function and will only be called if DeltaTime > 0
	check (DeltaTime > 0.0f);
	float InvDeltaTime = 1.0f / DeltaTime;

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_UpdatePreDynamics_Particles);

		const int32 NumParticles = SimulationState.Particles.Num();
		check(SimulationState.ParticleInfos.Num() == NumParticles);
		check(ParticleOwnerComponents.Num() == NumParticles);
		for (int32 Index = 0; Index != NumParticles; ++Index)
		{
			if (FRigDynamicsParticleComponent* PC = GetParticle(Hierarchy, ParticleOwnerComponents[Index]))
			{
				RigParticleSimulation::UpdateParticle(
					SimulationState, Index, PC->ParticleProperties.Mass,
					PC->ParticleProperties.GravityMultiplier, PC->ParticleProperties.Radius,
					PC->ParticleProperties.Damping, PC->ParticleProperties.bScaleDampingByInverseMass,
					PC->ParticleProperties.bCollideWithColliders);

				const FTransform TargetTM = GetGlobalTransform(Hierarchy, *PC);
				ParticleTargetTMs[Index] = TargetTM;
				const FSimVector TargetSimSpace =
					SimulationSpaceState.ConvertComponentSpacePositionToSimSpace(TargetTM.GetLocation());
				RigParticleSimulation::UpdateParticleTarget(
					SimulationState, Index, InvDeltaTime, TargetSimSpace,
					PC->ParticleProperties.Strength, PC->ParticleProperties.DampingRatio,
					PC->ParticleProperties.ExtraDamping, PC->ParticleProperties.TargetVelocityInfluence,
					PC->ParticleProperties.TargetMode, PC->ParticleProperties.AngleLimit,
					PC->ParticleProperties.AngleLimitStrength, PC->ParticleProperties.bAccelerationMode);

				// Consume and clear any forces queued by AddDynamicsParticleForce this frame. The
				// bone TM is the input pose (TargetTM above is the same global transform we want
				// for "Body" space). 
				if (!PC->PendingForces.IsEmpty())
				{
					if (SimulationState.Particles[Index].InvMass != 0.0f)
					{
						const float Mass = FMath::Max(PC->ParticleProperties.Mass, KINDA_SMALL_NUMBER);
						FSimVector Accumulator = FSimVector::ZeroVector;
						for (const FRigDynamicsParticleForce& Pending : PC->PendingForces)
						{
							FSimVector ForceSim = FSimVector::ZeroVector;
							switch (Pending.Space)
							{
							case EPhysicsControlSpace::World:
								ForceSim = SimulationSpaceState.ConvertWorldVectorToSimSpace(Pending.Force);
								break;
							case EPhysicsControlSpace::Component:
								ForceSim = SimulationSpaceState.ConvertComponentSpaceVectorToSimSpace(Pending.Force);
								break;
							case EPhysicsControlSpace::Body:
								ForceSim = SimulationSpaceState.ConvertComponentSpaceVectorToSimSpace(
									TargetTM.TransformVectorNoScale(Pending.Force));
								break;
							}

							// Convert to a "continuous-force equivalent". StepParticleDynamic
							// integrates Info.ExternalForce * Particle.InvMass over substeps; for a
							// frame-long continuous force F that sums to dV = (F/m) * frame_dt, so
							// Impulse J encoded as F = J/dt yields the correct dV = J/m. Position
							// drift differs from a true instantaneous impulse by O(substep_dt) but
							// velocity is exact.
							switch (Pending.Type)
							{
							case EPhysicsControlForceType::Force:
								Accumulator += ForceSim;
								break;
							case EPhysicsControlForceType::Acceleration:
								Accumulator += ForceSim * Mass;
								break;
							case EPhysicsControlForceType::Impulse:
								Accumulator += ForceSim * InvDeltaTime;
								break;
							case EPhysicsControlForceType::VelocityChange:
								Accumulator += ForceSim * (Mass * InvDeltaTime);
								break;
							}
						}
						SimulationState.ParticleInfos[Index].ExternalForce += Accumulator;
					}
					PC->PendingForces.Reset();
				}
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_UpdatePreDynamics_Constraints);

		// We call RigParticleSimulation::UpdateSkeletalConstraint here to support bone lengths
		// changing. This should be optional.
		const int32 NumSkeletalConstraints = SimulationState.SkeletalConstraints.Num();
		for (int32 Index = 0; Index != NumSkeletalConstraints; ++Index)
		{
			RigParticleSimulation::UpdateSkeletalConstraint(SimulationState, Index);
		}

		// Update user-specified hard constraints
		const int32 NumHardConstraints = SimulationState.HardDistanceConstraints.Num();
		check(NumHardConstraints == SimulationState.HardDistanceConstraintInfos.Num());
		check(HardDistanceConstraintOwnerComponents.Num() == SimulationState.HardDistanceConstraintInfos.Num());
		for (int32 Index = 0; Index != NumHardConstraints; ++Index)
		{
			if (const FRigDynamicsConstraintComponent* CC = GetConstraint(
				Hierarchy, HardDistanceConstraintOwnerComponents[Index]))
			{
				RigParticleSimulation::UpdateHardDistanceConstraint(
					SimulationState, Index, CC->LengthMultiplier, CC->ExtraLength);
			}
		}

		// Update user-specified soft constraints
		const int32 NumSoftConstraints = SimulationState.SoftDistanceConstraints.Num();
		check(NumSoftConstraints == SimulationState.SoftDistanceConstraintInfos.Num());
		check(SoftDistanceConstraintOwnerComponents.Num() == SimulationState.SoftDistanceConstraintInfos.Num());
		for (int32 Index = 0; Index != NumSoftConstraints; ++Index)
		{
			if (const FRigDynamicsConstraintComponent* CC = GetConstraint(
				Hierarchy, SoftDistanceConstraintOwnerComponents[Index]))
			{
				RigParticleSimulation::UpdateSoftDistanceConstraint(SimulationState, Index,
					CC->LengthMultiplier, CC->ExtraLength, CC->Strength, CC->DampingRatio, CC->ExtraDamping,
					CC->bAccelerationMode);
			}
		}

		// Update user-specified cone limits
		const int32 NumConeLimits = SimulationState.ConeLimits.Num();
		check(NumConeLimits == SimulationState.ConeLimitInfos.Num());
		check(ConeLimitOwnerComponents.Num() == SimulationState.ConeLimitInfos.Num());
		for (int32 Index = 0; Index != NumConeLimits; ++Index)
		{
			if (const FRigDynamicsConeLimitComponent* CC = GetConeLimit(
				Hierarchy, ConeLimitOwnerComponents[Index]))
			{
				RigParticleSimulation::UpdateConeLimit(
					SimulationState, Index, CC->Angle, CC->Strength, CC->DampingRatio);
			}
		}
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_UpdatePreDynamics_Colliders);

		// Update component-based colliders. Shape arrays are expected to match what was set up in
		// Instantiate (dynamic shape array changes are not yet supported, only size/position/orientation
		// changes).
		const int32 NumColliders = ColliderOwnerComponents.Num();
		check(NumColliders <= SimulationState.Colliders.Num());
		for (int32 Index = 0; Index != NumColliders; ++Index)
		{
			if (const FRigDynamicsColliderComponent* ColliderComponent = GetCollider(
				Hierarchy, ColliderOwnerComponents[Index]))
			{
				const FSimTransform ColliderSimSpace = SimulationSpaceState.ConvertComponentSpaceTransformToSimSpace(
					GetGlobalTransform(Hierarchy, *ColliderComponent));
				RigParticleSimulation::UpdateCollider(SimulationState, Index, ColliderSimSpace);

				RigParticleSimulation::FShapeCollection& Collider = SimulationState.Colliders[Index];

				// Now update the individual shapes (optional - only needed if things change)
				const int32 NumBoxes = ColliderComponent->Shapes.Boxes.Num();
				for (int32 BoxIndex = 0; BoxIndex != NumBoxes; ++BoxIndex)
				{
					if (ensure(Collider.BoxShapes.IsValidIndex(BoxIndex)))
					{
						const FRigDynamicsShapeBox& Box = ColliderComponent->Shapes.Boxes[BoxIndex];
						Collider.BoxShapes[BoxIndex].Update(FSimTransform(Box.TM), FSimVector(Box.Extents));
					}
				}
				const int32 NumCapsules = ColliderComponent->Shapes.Capsules.Num();
				for (int32 CapsuleIndex = 0; CapsuleIndex != NumCapsules; ++CapsuleIndex)
				{
					if (ensure(Collider.CapsuleShapes.IsValidIndex(CapsuleIndex)))
					{
						const FRigDynamicsShapeCapsule& Capsule = ColliderComponent->Shapes.Capsules[CapsuleIndex];
						Collider.CapsuleShapes[CapsuleIndex].Update(
							FSimTransform(Capsule.TM), Capsule.Length, Capsule.Radius);
					}
				}
				const int32 NumPlanes = ColliderComponent->Shapes.Planes.Num();
				for (int32 PlaneIndex = 0; PlaneIndex != NumPlanes; ++PlaneIndex)
				{
					if (ensure(Collider.PlaneShapes.IsValidIndex(PlaneIndex)))
					{
						const FRigDynamicsShapePlane& Plane = ColliderComponent->Shapes.Planes[PlaneIndex];
						Collider.PlaneShapes[PlaneIndex].Update(
							FSimTransform(Plane.TM), FSimVector2D(Plane.Extents));
					}
				}
			}
		}

		// Update confiners. Same transform / shape refresh logic as bone-attached colliders, plus
		// the per-confiner Strength cached on the simulation state.
		const int32 NumConfiners = ConfinerOwnerComponents.Num();
		check(NumConfiners == SimulationState.Confiners.Num());
		check(NumConfiners == SimulationState.ConfinerStrengths.Num());
		for (int32 Index = 0; Index != NumConfiners; ++Index)
		{
			if (const FRigDynamicsConfinerComponent* ConfinerComponent = GetConfiner(
				Hierarchy, ConfinerOwnerComponents[Index]))
			{
				const FSimTransform ConfinerSimSpace = SimulationSpaceState.ConvertComponentSpaceTransformToSimSpace(
					GetGlobalTransform(Hierarchy, *ConfinerComponent));
				RigParticleSimulation::UpdateConfiner(SimulationState, Index, ConfinerSimSpace);

				SimulationState.ConfinerStrengths[Index] = ConfinerComponent->Strength;

				RigParticleSimulation::FShapeCollection& Confiner = SimulationState.Confiners[Index];
				const int32 NumBoxes = ConfinerComponent->Shapes.Boxes.Num();
				for (int32 BoxIndex = 0; BoxIndex != NumBoxes; ++BoxIndex)
				{
					if (ensure(Confiner.BoxShapes.IsValidIndex(BoxIndex)))
					{
						const FRigDynamicsShapeBox& Box = ConfinerComponent->Shapes.Boxes[BoxIndex];
						Confiner.BoxShapes[BoxIndex].Update(FSimTransform(Box.TM), FSimVector(Box.Extents));
					}
				}
				const int32 NumCapsules = ConfinerComponent->Shapes.Capsules.Num();
				for (int32 CapsuleIndex = 0; CapsuleIndex != NumCapsules; ++CapsuleIndex)
				{
					if (ensure(Confiner.CapsuleShapes.IsValidIndex(CapsuleIndex)))
					{
						const FRigDynamicsShapeCapsule& Capsule = ConfinerComponent->Shapes.Capsules[CapsuleIndex];
						Confiner.CapsuleShapes[CapsuleIndex].Update(
							FSimTransform(Capsule.TM), Capsule.Length, Capsule.Radius);
					}
				}
				const int32 NumPlanes = ConfinerComponent->Shapes.Planes.Num();
				for (int32 PlaneIndex = 0; PlaneIndex != NumPlanes; ++PlaneIndex)
				{
					if (ensure(Confiner.PlaneShapes.IsValidIndex(PlaneIndex)))
					{
						const FRigDynamicsShapePlane& Plane = ConfinerComponent->Shapes.Planes[PlaneIndex];
						Confiner.PlaneShapes[PlaneIndex].Update(
							FSimTransform(Plane.TM), FSimVector2D(Plane.Extents));
					}
				}
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::UpdatePostDynamics(
	URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent, float Alpha)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_UpdatePostDynamics);

	// Particles are in depth order (root-to-leaf), ensured by SortParticlesRootToLeaf in Instantiate.
	// Note: branching chains where a parent is shared by multiple children could rotate the parent
	// twice. The multi-child case below handles this by only setting position, not orientation.
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		// Clear forces ready for the next update.
		SimulationState.ParticleInfos[Index].ExternalForce = FSimVector::ZeroVector;

		const RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];
		const RigParticleSimulation::FParticleInfo& ParticleInfo = SimulationState.ParticleInfos[Index];

		if (ParticleInfo.NumChildren > 1)
		{
			// When there are multiple children, we always:
			// 1. track the particle position exactly
			// 2. Maintain the original component-space orientation.
			const int32 BoneIndex = GetElementIndex(Hierarchy, ParticleOwnerComponents[Index]);
			FTransform OriginalTM = Hierarchy.GetGlobalTransform(BoneIndex);
			const FVector ParticlePosComponentSpace =
				SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(Particle.Position);
			FTransform ParticleTM(ParticleTargetTMs[Index].GetRotation(), ParticlePosComponentSpace);
			FTransform BlendedTM;
			BlendedTM.Blend(OriginalTM, ParticleTM, Alpha);
			Hierarchy.SetGlobalTransform(BoneIndex, BlendedTM);
		}
		else
		{
			bool bForceSettingPosition = false;
			if (SolverComponent.Settings.bReadBoneOrientations)
			{
				const int32 ParentIndex = ParticleInfo.ParentParticleIndex;

				// Rotate parent bone so it points toward this particle
				if (ParentIndex != INDEX_NONE)
				{
					const RigParticleSimulation::FParticleInfo& ParentParticleInfo = 
						SimulationState.ParticleInfos[ParentIndex];

					if (Particle.InvMass == 0.0f && SimulationState.Particles[ParentIndex].InvMass == 0.0f)
					{
						// If both this particle and its parent are kinematic, neither moved during
						// simulation, so the parent bone's orientation is already correct from animation.
					}
					else
					{
						// Only rotate joints when there is a single child. Alternatively, we could just
						// apply a fraction of the delta rotation, but this would be tricky as we want to
						// update incrementally.
						if (ParentParticleInfo.NumChildren != 1)
						{
							bForceSettingPosition = true;
						}
						else
						{
							const FVector ChildParticlePos = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
								Particle.Position);
							const FVector ParentParticlePos = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
								SimulationState.Particles[ParentIndex].Position);
							const FVector DirectionFromParticles = ChildParticlePos - ParentParticlePos;

							const int32 ParentBoneIndex = GetElementIndex(Hierarchy, ParticleOwnerComponents[ParentIndex]);
							const int32 ChildBoneIndex = GetElementIndex(Hierarchy, ParticleOwnerComponents[Index]);

							FTransform ParentBoneTransform =   Hierarchy.GetGlobalTransform(ParentBoneIndex);

							// When the parent particle's bone is the direct skeleton parent we can derive the
							// child's global position from the (already known) parent transform + the child's
							// local transform, avoiding a GetGlobalTransform call that would recompute dirty globals.
							const FVector ChildBonePos = ParticleInfo.bParentParticleIsDirectParent
								? ParentBoneTransform.TransformPositionNoScale(
									Hierarchy.GetLocalTransform(ChildBoneIndex).GetLocation())
								: FVector(Hierarchy.GetGlobalTransform(ChildBoneIndex).GetLocation());

							const FVector DirectionFromBones = ChildBonePos - ParentBoneTransform.GetLocation();

							const FQuat DeltaRotation = FQuat::FindBetweenVectors(
								DirectionFromBones, DirectionFromParticles);
							const FQuat BlendedRotation = FQuat::Slerp(FQuat::Identity, DeltaRotation, Alpha);
							ParentBoneTransform.SetRotation(BlendedRotation * ParentBoneTransform.GetRotation());
							Hierarchy.SetGlobalTransform(ParentBoneIndex, ParentBoneTransform);
						}
					}
				}
			}

			// Position fixup
			if (SolverComponent.Settings.bReadBonePositions || bForceSettingPosition)
			{
				const int32 BoneIndex = GetElementIndex(Hierarchy, ParticleOwnerComponents[Index]);
				FTransform Transform = Hierarchy.GetGlobalTransform(BoneIndex);
				const FVector ParticlePosCS = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
					Particle.Position);
				const FVector BlendedPosition = FMath::Lerp(Transform.GetLocation(), ParticlePosCS, Alpha);
				Transform.SetLocation(BlendedPosition);
				Hierarchy.SetGlobalTransform(BoneIndex, Transform);
			}
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::ResetPoseFromAnimation(const URigHierarchy& Hierarchy)
{
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		RigParticleSimulation::FParticleInfo& Info = SimulationState.ParticleInfos[Index];
		RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];

		const FTransform GlobalTM = GetGlobalTransform(Hierarchy, ParticleOwnerComponents[Index]);

		Info.TargetPosition = SimulationSpaceState.ConvertComponentSpacePositionToSimSpace(GlobalTM.GetLocation());;
		Info.PreviousTargetPosition = Info.TargetPosition;
		Info.TargetVelocity = FSimVector::ZeroVector;

		Particle.Position = Info.TargetPosition;
		Particle.PrevPosition = Info.TargetPosition;
	}

	SimulationState.PrevStepTime = 0.0f;
}

//======================================================================================================================
bool FRigDynamicsSolver::AnyKinematicExceedsSpeed(const float Threshold) const
{
	const double ThresholdSq = double(Threshold) * double(Threshold);
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		const RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];
		if (Particle.InvMass != 0.0f)
		{
			continue;
		}
		const RigParticleSimulation::FParticleInfo& Info = SimulationState.ParticleInfos[Index];
		if (double(Info.TargetVelocity.SizeSquared()) > ThresholdSq)
		{
			return true;
		}
	}
	return false;
}

//======================================================================================================================
bool FRigDynamicsSolver::AnyParticleExceedsDistanceFromOrigin(const float Threshold) const
{
	const double ThresholdSq = double(Threshold) * double(Threshold);
	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		const RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];
		if (double(Particle.Position.SizeSquared()) > ThresholdSq)
		{
			return true;
		}
	}
	return false;
}

//======================================================================================================================
void FRigDynamicsSolver::ClearParticlePendingForces(URigHierarchy& Hierarchy)
{
	for (FCachedRigComponent& Cached : ParticleOwnerComponents)
	{
		if (FRigDynamicsParticleComponent* PC = GetParticle(Hierarchy, Cached))
		{
			PC->PendingForces.Reset();
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::AddFieldForcesToParticles(
	URigHierarchy&                     Hierarchy,
	const FRigDynamicsSolverComponent& SolverComponent,
	const ERigDynamicsSimulationSpace  FieldSpace,
	const FTransform&                  FieldTransform,
	const EPhysicsControlForceType     Type,
	const float                        RadialForce,
	const FRichCurve*                  RadialMultiplier,
	const FVector&                     LinearForceDirection,
	const float                        LinearForce,
	const FRichCurve*                  LinearMultiplier,
	const FVector&                     RotationalForceAxis,
	const float                        RotationalForce,
	const FRichCurve*                  RotationalMultiplier,
	FRigVMDrawInterface*               DrawInterface,
	const bool                         bDrawDebug,
	const float                        DebugForceScale)
{
	const int32 NumParticles = SimulationState.Particles.Num();

	// Field extents come from the FieldTransform's scale; degenerate axes mean no ellipsoid.
	const FVector Radii = FieldTransform.GetScale3D();
	if (Radii.X < KINDA_SMALL_NUMBER || Radii.Y < KINDA_SMALL_NUMBER || Radii.Z < KINDA_SMALL_NUMBER)
	{
		return;
	}

	// Resolve the field into component space (rotation + translation only - Radii holds extents).
	FTransform FieldCompTM;
	switch (FieldSpace)
	{
	case ERigDynamicsSimulationSpace::World:
		FieldCompTM = FieldTransform.GetRelativeTransform(SimulationSpaceState.GetComponentTM());
		break;
	case ERigDynamicsSimulationSpace::Component:
		FieldCompTM = FieldTransform;
		break;
	case ERigDynamicsSimulationSpace::SpaceBone:
		FieldCompTM = FieldTransform * Hierarchy.GetGlobalTransform(SolverComponent.Settings.SpaceBone);
		break;
	}
	const FQuat FieldCompRotation = FieldCompTM.GetRotation();

	// LinearForceDirection / RotationalForceAxis are auto-normalised so non-unit input doesn't
	// double-encode magnitude. The cross-product tangent is normalised inside the loop.
	const FVector LinearDirLocal = LinearForceDirection.GetSafeNormal();

	auto EvalCurve = [](const FRichCurve* Curve, float X)
	{
		return Curve ? Curve->Eval(X, 1.0f) : 1.0f;
	};

	const bool bDraw = ShouldDrawForceFieldDebug(bDrawDebug) && DrawInterface != nullptr;
	const float EffectiveDebugForceScale = GetForceFieldDebugScale(DebugForceScale);
	if (bDraw)
	{
		DrawForceFieldEllipsoid(DrawInterface, FieldCompTM, Radii);
	}

	// Add the per-particle forces.
	for (int32 i = 0; i < NumParticles; ++i)
	{
		const FVector ParticleCompPos =
			SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(SimulationState.Particles[i].Position);
		// Particle position relative to the field
		const FVector ParticleLocal = FieldCompTM.InverseTransformPositionNoScale(ParticleCompPos);
		// Normalized position in the field's space
		const FVector ParticleNorm(ParticleLocal.X / Radii.X, ParticleLocal.Y / Radii.Y, ParticleLocal.Z / Radii.Z);
		const float RadiusFraction = static_cast<float>(ParticleNorm.Size());
		if (RadiusFraction > 1.0f)
		{
			continue;
		}

		// Radial: zero direction at field centre falls out naturally via GetSafeNormal.
		const FVector RadialDir = ParticleLocal.GetSafeNormal();
		const FVector RadialContrib = RadialDir * (RadialForce * EvalCurve(RadialMultiplier, RadiusFraction));

		const FVector LinearContrib = LinearDirLocal * (LinearForce * EvalCurve(LinearMultiplier, RadiusFraction));

		// Rotational: zero on the rotation axis (cross-product is zero) - falls out via GetSafeNormal.
		const FVector Tangent = FVector::CrossProduct(RotationalForceAxis, ParticleLocal).GetSafeNormal();
		const FVector RotContrib = Tangent * (RotationalForce * EvalCurve(RotationalMultiplier, RadiusFraction));

		const FVector ForceLocal = RadialContrib + LinearContrib + RotContrib;
		const FVector ForceComp = FieldCompRotation.RotateVector(ForceLocal);

		if (FRigDynamicsParticleComponent* PC = GetParticle(Hierarchy, ParticleOwnerComponents[i]))
		{
			PC->PendingForces.Emplace(ForceComp, EPhysicsControlSpace::Component, Type);
		}

		if (bDraw)
		{
			DrawForceFieldArrow(DrawInterface, ParticleCompPos, ForceComp * EffectiveDebugForceScale);
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::TrackParticlePositions(const URigHierarchy& Hierarchy, const float DeltaTime)
{
	// We're an internal function and will only be called if DeltaTime > 0
	check (DeltaTime > 0.0f);
	const float InvDeltaTime = 1.0f / DeltaTime;

	const int32 NumParticles = SimulationState.Particles.Num();
	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		RigParticleSimulation::FParticleInfo& Info = SimulationState.ParticleInfos[Index];
		RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];

		const FTransform GlobalTM = GetGlobalTransform(Hierarchy, ParticleOwnerComponents[Index]);

		Info.PreviousTargetPosition = Info.TargetPosition;
		Info.TargetPosition = SimulationSpaceState.ConvertComponentSpacePositionToSimSpace(GlobalTM.GetLocation());
		Info.TargetVelocity = (Info.TargetPosition - Info.PreviousTargetPosition) * InvDeltaTime;

		Particle.PrevPosition = Particle.Position;
		Particle.Position = Info.TargetPosition;
	}

	SimulationState.PrevStepTime = DeltaTime;
}

//======================================================================================================================
void FRigDynamicsSolver::UpdateSimulationSpaceStateTransforms(
	const FRigVMExecuteContext&        ExecuteContext,
	const URigHierarchy&               Hierarchy,
	const FRigDynamicsSolverComponent& SolverComponent,
	const float                        DeltaTime,
	FRigDynamicsSimulationSpaceState&  OutState,
	const bool                         bReset)
{
	const FRigDynamicsSolverSettings& SolverSettings = SolverComponent.Settings;

	FTransform ComponentTM = ExecuteContext.GetToWorldSpaceTransform();

	FTransform SimulationSpaceTM;
	switch (SolverSettings.SimulationSpace)
	{
	case ERigDynamicsSimulationSpace::World:
	{
		// Identity: simulation space IS world space
		break;
	}
	case ERigDynamicsSimulationSpace::Component:
	{
		SimulationSpaceTM = ComponentTM;
		break;
	}
	case ERigDynamicsSimulationSpace::SpaceBone:
	{
		if (SolverSettings.SpaceBone.IsValid())
		{
			SimulationSpaceTM = Hierarchy.GetGlobalTransform(SolverSettings.SpaceBone) * ComponentTM;
		}
		else
		{
			SimulationSpaceTM = ComponentTM;
		}
		break;
	}
	}

	if (bReset)
	{
		OutState.Reset(ComponentTM, SimulationSpaceTM);
	}
	else
	{
		OutState.Update(SolverComponent.TeleportDetection, ComponentTM, SimulationSpaceTM, DeltaTime);
	}
}

//======================================================================================================================
void FRigDynamicsSolver::StepSimulation(
	const FRigVMExecuteContext&        ExecuteContext,
	URigHierarchy&                     Hierarchy,
	const FRigDynamicsSolverComponent& SolverComponent,
	const AActor*                      OwningActorPtr,
	const float                        DeltaTimeOverride,
	const float                        SimulationSpaceDeltaTimeOverride,
	const float                        Alpha,
	const bool                         bTrackVelocitiesDuringPassThrough)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_StepSimulation);

	float DeltaTime = ExecuteContext.GetDeltaTime();
	if (DeltaTimeOverride > 0.0f)
	{
		DeltaTime = DeltaTimeOverride;
	}
	else if (DeltaTimeOverride < 0.0f)
	{
		DeltaTime = 0.0f;
	}

	float SimulationSpaceDeltaTime = DeltaTime;
	if (SimulationSpaceDeltaTimeOverride > 0.0f)
	{
		SimulationSpaceDeltaTime = SimulationSpaceDeltaTimeOverride;
	}

	// Capture the interval since the previous evaluation before we overwrite the tracker. A
	// negative value on the first frame after instantiation means "no prior evaluation" and skips
	// the test.
	const double CurrentAbsoluteTime = ExecuteContext.GetAbsoluteTime();
	const double PreviousAbsoluteTime = LastEvaluationAbsoluteTime;
	LastEvaluationAbsoluteTime = CurrentAbsoluteTime;

	// Pass-through when alpha is zero: bypass the full simulation
	if (Alpha <= 0.0f)
	{
		if (bTrackVelocitiesDuringPassThrough)
		{
			// If DeltaTime <= 0 we can't compute velocities, so skip tracking this frame. The
			// existing particle positions and implicit Verlet velocities are left as-is, so when
			// simulation time resumes (DeltaTime > 0) the next tracking or simulation step picks up
			// from the prior state.
			if (DeltaTime > 0.0f)
			{
				// Tracking mode: keep particle positions and sim-space state up to date so that
				// velocities are valid when alpha increases.
				if (bNeedToInstantiate)
				{
					Instantiate(ExecuteContext, Hierarchy, SolverComponent);
				}
				UpdateSimulationSpaceStateTransforms(
					ExecuteContext, Hierarchy, SolverComponent, SimulationSpaceDeltaTime, SimulationSpaceState);
				SimulationSpaceMotion = SimulationSpaceState.CalculateMotion(
					SolverComponent.SpaceMotion, ExecuteContext.GetAbsoluteTime());
				TrackParticlePositions(Hierarchy, DeltaTime);

				// Tracking maintains valid Verlet velocities, so clear any pending reset that may
				// have been set by a prior non-tracking pass-through frame.
				bNeedToResetPose = false;
			}
		}
		else if (!bNeedToInstantiate)
		{
			// Non-tracking mode: mark for a pose reset so the simulation snaps to the current
			// animation pose with zero velocity when alpha increases. Avoids the cost of a full re-instantiation.
			bNeedToResetPose = true;
		}

		// Forces queued for "this frame" are dropped: we are not simulating, so there is no
		// frame to apply them to. Run after the conditional Instantiate above so that
		// ParticleOwnerComponents is populated even on the very first frame, preventing queued
		// forces from leaking into a later non-pass-through step.
		ClearParticlePendingForces(Hierarchy);
		return;
	}

	// Full simulation path
	bKinematicSpeedResetInLastStep = false;
	bPositionResetInLastStep = false;
	bEvaluationIntervalResetInLastStep = false;

	if (bNeedToInstantiate)
	{
		Instantiate(ExecuteContext, Hierarchy, SolverComponent);
	}

	// Evalualation interval reset. If the rig wasn't ticked for a while (and the interval is
	// significantly larger than this frame's delta time), reset to the input pose so we don't
	// integrate across the discontinuity. A -ve value for PreviousAbsoluteTime indicates it's from
	// the first frame/hasn't been updated yet.
	if (SolverComponent.Settings.bResetFromEvaluationInterval && PreviousAbsoluteTime >= 0.0)
	{
		// Slightly more than the typical variability between absolute delta times and the frame delta time
		double constexpr DeltaTimeMargin = 0.01;
		const double Interval = CurrentAbsoluteTime - PreviousAbsoluteTime;
		if (Interval > SolverComponent.Settings.EvaluationIntervalThresholdForReset && 
			Interval > (DeltaTime + DeltaTimeMargin))
		{
			UE_LOGF(LogRigDynamics, Log, "Evaluation interval %f triggered reset", Interval);
			bNeedToResetPose = true;
			bEvaluationIntervalResetInLastStep = true;
		}
	}

	if (bNeedToResetPose)
	{
		// Reset the simulation space state (zeroes stale velocities/accelerations) and snap all
		// particles to the current animation pose with zero velocity.
		UpdateSimulationSpaceStateTransforms(
			ExecuteContext, Hierarchy, SolverComponent, 0.0f, SimulationSpaceState, /*bReset=*/true);
		ResetPoseFromAnimation(Hierarchy);
		bNeedToResetPose = false;
	}

	UpdateSimulationSpaceStateTransforms(
		ExecuteContext, Hierarchy, SolverComponent, SimulationSpaceDeltaTime, SimulationSpaceState);

	SimulationSpaceMotion = SimulationSpaceState.CalculateMotion(
		SolverComponent.SpaceMotion, ExecuteContext.GetAbsoluteTime());

	const FRigDynamicsSolverSettings& SolverSettings = SolverComponent.Settings;
	RigParticleSimulation::FSolverSettings ParticleSolverSettings;
	ParticleSolverSettings.Gravity = SimulationSpaceState.ConvertWorldVectorToSimSpace(SolverSettings.Gravity);
	ParticleSolverSettings.NumIterations = SolverSettings.NumIterations;
	ParticleSolverSettings.NumConstraintSubIterations = SolverSettings.NumConstraintSubIterations;
	ParticleSolverSettings.MaxTimeStep = CVarControlRigDynamicsMaxTimeStepOverride.GetValueOnAnyThread() < 0
		? SolverSettings.MaxTimeStep : CVarControlRigDynamicsMaxTimeStepOverride.GetValueOnAnyThread();
	ParticleSolverSettings.MaxNumSteps = CVarControlRigDynamicsMaxNumStepsOverride.GetValueOnAnyThread() < 0
		? SolverSettings.MaxNumSteps : CVarControlRigDynamicsMaxNumStepsOverride.GetValueOnAnyThread();

	if (DeltaTime > 0.0f)
	{
		UpdatePreDynamics(Hierarchy, SolverComponent, DeltaTime);

		// Pre-step reset: if any kinematic particle's sim-space target speed has spiked (e.g. from
		// an animation teleport), zero every particle's velocity so the spike doesn't propagate
		// into the dynamic neighbours via the constraint solver.
		if (SolverSettings.bResetFromKinematicSpeed &&
			AnyKinematicExceedsSpeed(SolverSettings.KinematicSpeedThresholdForReset))
		{
			UE_LOGF(LogRigDynamics, Log, "Kinematic speed triggered reset");
			ResetPoseFromAnimation(Hierarchy);
			bKinematicSpeedResetInLastStep = true;
		}

		RigParticleSimulation::Simulate(SimulationState, ParticleSolverSettings, SimulationSpaceMotion, DeltaTime);

		// Post-step reset: if any particle has blown out past the distance threshold, snap the
		// whole simulation back to the animation pose with zero velocity before the writeback.
		if (SolverSettings.bResetFromPosition &&
			AnyParticleExceedsDistanceFromOrigin(SolverSettings.PositionThresholdForReset))
		{
			UE_LOGF(LogRigDynamics, Log, "Distance from origin triggered reset");
			ResetPoseFromAnimation(Hierarchy);
			bPositionResetInLastStep = true;
		}
	}
	else
	{
		// Non-pass-through but no time advances this frame: UpdatePreDynamics (which clears the
		// queue normally) does not run, so discard pending forces here to keep the "queued forces
		// are for this frame" contract.
		ClearParticlePendingForces(Hierarchy);
	}

	// Always do a read back, even if DeltaTime is zero
	UpdatePostDynamics(Hierarchy, SolverComponent, Alpha);
}
