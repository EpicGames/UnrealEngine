// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsHelpers.h"

#include "RigParticleSimulation.h"
#include "RigDynamicsSimulationSpace.h"

#include "Rigs/RigHierarchyCache.h"

using RigParticleSimulation::FSimVector;
using RigParticleSimulation::FSimTransform;
using RigParticleSimulation::FSimVector2D;

struct FRigVMExecuteContext;
struct FRigVMDrawInterface;
struct FRichCurve;
class URigHierarchy;

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
DECLARE_LOG_CATEGORY_EXTERN(LogRigDynamics, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogRigDynamics, Log, All);
#endif

//======================================================================================================================
// True if any per-element draw would emit geometry OR the per-particle text overlay would draw, given
// the rig's bShow flags + ParticleValueDisplay AND the per-element ControlRig.Dynamics.Show*Override
// & ParticleValueDisplayOverride CVars
//======================================================================================================================
bool RigDynamicsShouldVisualize(const FRigDynamicsVisualizationSettings& VisualizationSettings);

//======================================================================================================================
// Returns the effective per-call bDrawDebug for the force-field node, after applying the
// ControlRig.Dynamics.ShowForceFieldsOverride CVar (-1 = use SettingsValue, 0/1 = force).
//======================================================================================================================
bool ShouldDrawForceFieldDebug(bool SettingsValue);

//======================================================================================================================
// Returns the effective DebugForceScale for the force-field node, after applying the
// ControlRig.Dynamics.ForceFieldDebugScaleOverride CVar (negative = use SettingsValue).
//======================================================================================================================
float GetForceFieldDebugScale(float SettingsValue);

//======================================================================================================================
// Traces three orthogonal ellipses (XY/YZ/XZ planes in the pose's local frame) using DrawLineStrip.
// Detail is per-ring segment count. No-op if DI is null or the ControlRig.Dynamics.AllowVisualization
// CVar is disabled.
//======================================================================================================================
void DrawForceFieldEllipsoid(
	FRigVMDrawInterface* DI, const FTransform& Pose, const FVector& Radii,
	float Thickness = 0.0f, int32 Detail = 24);

//======================================================================================================================
// Draws an arrow from Origin along Vector (Vector is in the same component space as Origin). Barbs
// scale with arrow length. No-op if DI is null, the CVar is disabled, or Vector is near-zero.
//======================================================================================================================
void DrawForceFieldArrow(
	FRigVMDrawInterface* DI, const FVector& Origin, const FVector& Vector,
	float Thickness = 0.0f);

//======================================================================================================================
// This is the main solver for Dynamics, handling the particles and constraints, as well as reading
// from/writing back to the hierarchy.
//======================================================================================================================
struct FRigDynamicsSolver
{
public:
	FRigDynamicsSolver(const FName InOwnerName);

	// This will initialise/create the simulation and then create everything we need in it. 
	void Instantiate(
		const FRigVMExecuteContext&        ExecuteContext, 
		const URigHierarchy&               Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent);

	// Integrates the simulation forwards.
	// If DeltaTimeOverride is +ve, then that value is used.
	// If it is zero, then delta time is taken from the execute context
	// If it is negative, then the simulation isn't stepped.
	void StepSimulation(
		const FRigVMExecuteContext&        ExecuteContext,
		URigHierarchy&                     Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent,
		const AActor*                      OwningActorPtr,
		const float                        DeltaTimeOverride,
		const float                        SimulationSpaceDeltaTimeOverride,
		const float                        Alpha,
		const bool                         bTrackVelocitiesDuringPassThrough);

	// Draws shapes etc.
	void Draw(
		FRigVMDrawInterface*                     DI,
		const URigHierarchy&                     Hierarchy,
		const UWorld*                            DebugWorld,
		const FRigDynamicsVisualizationSettings& VisualizationSettings) const;

	// Read-only access to the simulation-space state populated by the most recent StepSimulation
	// (or tracking pass-through). Used by inspection nodes.
	const FRigDynamicsSimulationSpaceState& GetSimulationSpaceState() const { return SimulationSpaceState; }

	// Adds a per-particle FRigDynamicsParticleForce on every simulated particle that lies inside
	// the given ellipsoidal field, computed from a radial + linear + rotational contribution shaped
	// by the supplied per-radius-fraction curves. The field's transform's translation+rotation
	// gives the ellipsoid centre and orientation. The transform's scale gives the per-axis radii.
	// Particles outside the ellipsoid are not affected. Passing nullptr for any curve makes that
	// contribution a flat 1.0.
	void AddFieldForcesToParticles(
		URigHierarchy&                     Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent,
		const ERigDynamicsSimulationSpace  FieldSpace,
		const FTransform&                  FieldTransform,
		const EPhysicsControlForceType     Type,
		const float                        RadialForce,
		const FRichCurve*                  RadialMultiplier,
		const FVector&                     LinearForceDirection,
		const float                        LinearForce,
		const FRichCurve*                  LinearMultiplier,
		const FVector&                     RotationalForceAxis,
		const float                        RotationalForce,
		const FRichCurve*                  RotationalMultiplier,
		FRigVMDrawInterface*               DrawInterface,
		const bool                         bDrawDebug,
		const float                        DebugForceScale);

	// True if KinematicSpeedThresholdForReset tripped during the most recent full StepSimulation.
	bool WasKinematicSpeedResetInLastStep() const { return bKinematicSpeedResetInLastStep; }

	// True if PositionThresholdForReset tripped during the most recent full StepSimulation.
	bool WasPositionResetInLastStep() const { return bPositionResetInLastStep; }

	// True if a long interval between solver evaluations was detected during the most recent full
	// StepSimulation (gated by bResetFromEvaluationInterval / EvaluationIntervalThresholdForReset
	// on the solver).
	bool WasEvaluationIntervalResetInLastStep() const { return bEvaluationIntervalResetInLastStep; }

private:
	// Returns the index into ComponentInfos/Particles for the given element, or INDEX_NONE
	int32 GetParticleIndexSlow(const FRigBaseElement& Element, const URigHierarchy& Hierarchy);

	// Returns the index into ComponentInfos/Particles for the given element, or INDEX_NONE
	int32 GetParticleIndexSlow(const FRigComponentKey& ComponentKey) const;

	// Returns the index into ComponentInfos/Particles for the parent of the given element, or INDEX_NONE
	int32 GetParentParticleIndexSlow(const FRigBaseElement& Element, const URigHierarchy& Hierarchy);

	// Returns the index into ComponentInfos/Particles for the parent of the given particle, or INDEX_NONE
	int32 GetParentParticleIndexSlow(int32 ParticleIndex, const URigHierarchy& Hierarchy);

	// This will runs an update prior to the simulation update - e.g. read from the hierarchy target
	// positions etc. Requires DeltaTime > 0. Hierarchy is non-const because the pending forces need
	// to be cleared from the particle components' arrays.
	void UpdatePreDynamics(
		URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent, float DeltaTime);

	// Discards every particle's pending force queue without applying any of it. Called from
	// StepSimulation on paths where UpdatePreDynamics will not run (pass-through frames, or frames
	// with DeltaTime <= 0) so that queued forces do not accumulate into the next simulating frame.
	void ClearParticlePendingForces(URigHierarchy& Hierarchy);

	// This will read back from the simulation into the hierarchy
	void UpdatePostDynamics(URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent, float Alpha);

	// Lightweight position tracking for pass-through mode (Alpha <= 0). Snaps particles to their
	// hierarchy targets so that implicit Verlet velocities remain valid for smooth resume. Requires
	// DeltaTime > 0.
	void TrackParticlePositions(const URigHierarchy& Hierarchy, float DeltaTime);

	// Snaps all particles to the current animation pose with zero velocity. Used when resuming
	// from a non-tracking pass-through, or after a reset. Assumes the simulation space state
	// has already been reset/updated for coordinate conversion.
	void ResetPoseFromAnimation(const URigHierarchy& Hierarchy);

	// Instantiate helpers, called by Instantiate() in order.
	void InstantiateColliders(
		const URigHierarchy& Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent,
		const FRigDynamicsSimulationSpaceState& SpaceState);

	// Mirrors InstantiateColliders for confiners. Must run before InstantiateParticles so that
	// particles can resolve their Confiners keys into simulation-state indices.
	void InstantiateConfiners(
		const URigHierarchy& Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent,
		const FRigDynamicsSimulationSpaceState& SpaceState);

	void InstantiateParticles(
		const URigHierarchy& Hierarchy,
		const FRigDynamicsSolverComponent& SolverComponent,
		const FRigDynamicsSimulationSpaceState& SpaceState);

	// Resolves each particle's parent particle index (walking the hierarchy), sets NumChildren
	// and bParentParticleIsDirectParent on the parent info, and forces root-most particles to
	// kinematic. Runs before SortParticles.
	void ComputeParticleParents(const URigHierarchy& Hierarchy);

	// Root-to-leaf BFS sort of particles, with remapping of the solver-owned parallel arrays
	// (ParticleTargetTMs, ParticleOwnerComponents). Constraint / cone-limit / collision indices
	// inside SimulationState are remapped by SortParticlesRootToLeaf itself.
	void SortParticles();

	// Post-sort: auto-disables collision between a particle and any collider on the same element
	// or the parent element, and resolves the user-specified particle-to-particle collision pairs.
	void SetParticlesNoCollision(const URigHierarchy& Hierarchy);

	// Creates one skeletal distance constraint per non-root particle (between the particle and
	// its parent particle). Constraint indices are in post-sort order.
	void InstantiateSkeletalConstraints(const URigHierarchy& Hierarchy);

	// Creates user-defined hard and soft distance constraints from the solver component.
	void InstantiateDistanceConstraints(
		const URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent);

	// Creates user-defined cone limits from the solver component.
	void InstantiateConeLimits(
		const URigHierarchy& Hierarchy, const FRigDynamicsSolverComponent& SolverComponent);

	// Sorts cone limits by child index (root-to-leaf) for better iterative convergence.
	void SortConeLimits();

	// True if any particle with InvMass == 0 has Info.TargetVelocity.Size() > Threshold.
	// Short-circuits on first match. Caller should only invoke when Threshold > 0.
	bool AnyKinematicExceedsSpeed(float Threshold) const;

	// True if any particle's simulation-space distance from the origin exceeds Threshold.
	// Short-circuits on first match. Caller should only invoke when Threshold > 0.
	bool AnyParticleExceedsDistanceFromOrigin(float Threshold) const;

	// Computes the component and simulation-space transforms from the solver settings and calls
	// OutState.Update(). When DeltaTime > 0, velocities and accelerations are derived from the
	// position change. When DeltaTime == 0, only the transforms are set. When bReset is true,
	// the state is fully reset (zeroing velocities/accelerations) - used after teleports or when
	// resuming from a pass-through that wasn't tracking.
	static void UpdateSimulationSpaceStateTransforms(
		const FRigVMExecuteContext&          ExecuteContext,
		const URigHierarchy&                 Hierarchy,
		const FRigDynamicsSolverComponent&   SolverComponent,
		float                                DeltaTime,
		FRigDynamicsSimulationSpaceState&    OutState,
		bool                                 bReset = false);

private:

	// A name that identifies what owns us - just used for debugging/logging
	FName OwnerName;

	// Instantiation can be explicit, or triggered automatically on the StepSimulation call - track
	// whether it is needed.
	bool bNeedToInstantiate = true;

	// When true, the simulation state will be snapped to the current animation pose with zero
	// velocity on the next step. Cheaper than a full re-instantiation. Set when resuming from a
	// non-tracking pass-through, and will also be used for teleport handling.
	bool bNeedToResetPose = false;

	// Observation flags set by the most recent full StepSimulation, cleared at the start of each.
	// Exposed read-only via the Was*InLastStep accessors and surfaced through the Get-data node.
	bool bKinematicSpeedResetInLastStep = false;
	bool bPositionResetInLastStep = false;
	bool bEvaluationIntervalResetInLastStep = false;

	// Absolute time (from the rig's ExecuteContext) at the end of the most recent StepSimulation
	// call. A value < 0 means "no prior evaluation" - the interval test is skipped on the first
	// frame after instantiation. Populated regardless of bResetFromEvaluationInterval so toggling
	// the flag on doesn't have a one-frame priming delay.
	double LastEvaluationAbsoluteTime = -1.0;

	// Each array here here is paired with the relevant one in the simulation state. Infos contain
	// data that refers to control rig, and is used to process inputs. The simulation state data are
	// used by the simulation, and should be packed as tightly as possible. I have tried baking the
	// arrays to pack them into contiguous memory, but it made no measurable improvement.
	//
	// Note that the simulation might have more objects than we have - but the indexing between
	// these and those should still match.
	TArray<FTransform>           ParticleTargetTMs;
	TArray<FCachedRigComponent>  ParticleOwnerComponents;
	TArray<FCachedRigComponent>  HardDistanceConstraintOwnerComponents;
	TArray<FCachedRigComponent>  SoftDistanceConstraintOwnerComponents;
	TArray<FCachedRigComponent>  ColliderOwnerComponents;
	TArray<FCachedRigComponent>  ConfinerOwnerComponents;
	TArray<FCachedRigComponent>  ConeLimitOwnerComponents;

	RigParticleSimulation::FSimulationState SimulationState;

	FRigDynamicsSimulationSpaceState SimulationSpaceState;
	RigParticleSimulation::FSimulationSpaceMotion SimulationSpaceMotion;
};
