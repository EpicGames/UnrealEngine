// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsSolverComponent.h"

#include "RigDynamicsSolver.h"
#include "RigDynamicsObjectVersion.h"

// Defined in RigDynamicsData.cpp - drains a pre-SimulationSpaceRegrouping
// FRigDynamicsSimulationSpaceSettings record from the archive, discarding all values.
extern void DrainLegacySimulationSpaceSettings(FArchive& Ar);

//======================================================================================================================
void FRigDynamicsSolverComponent::Serialize(FArchive& Ar)
{
	Ar << Particles;
	Ar << Colliders;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::Constraints)
	{
		Ar << Constraints;
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::ConeLimits)
	{
		Ar << ConeLimits;
	}
	Ar << Settings;
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::SimulationSpaceRegrouping)
	{
		// New layout: simulation-space conditioning (with drag and inertial forces nested) plus
		// teleport detection as a peer.
		Ar << SpaceMotion;
		Ar << TeleportDetection;
	}
	else
	{
		// Old layout: a flat FRigDynamicsSimulationSpaceSettings followed by a peer
		// FRigDynamicsSimulationDragSettings. Drain the bytes - values are discarded; the
		// new fields keep their constructor defaults. Existing test assets need re-saving.
		if (Ar.IsLoading())
		{
			if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::SimulationSpace)
			{
				DrainLegacySimulationSpaceSettings(Ar);
			}
			if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::DragSettings)
			{
				FRigDynamicsSimulationDragSettings LegacyDragSettings;
				Ar << LegacyDragSettings;
			}
		}
		// Save path can't reach here: UsingCustomVersion always writes at LatestVersion.
	}
	if (Ar.CustomVer(FRigDynamicsObjectVersion::GUID) >= FRigDynamicsObjectVersion::Confiners)
	{
		Ar << Confiners;
	}
}

//======================================================================================================================
void FRigDynamicsSolverComponent::Save(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRigDynamicsObjectVersion::GUID);
	FRigBaseComponent::Save(Ar);
	Serialize(Ar);
}

//======================================================================================================================
void FRigDynamicsSolverComponent::Load(FArchive& Ar)
{
	FRigBaseComponent::Load(Ar);
	Serialize(Ar);
}

//======================================================================================================================
FRigDynamicsSolver* FRigDynamicsSolverComponent::GetDynamicsSolver() const
{
	if (!DynamicsSolverPtr.IsValid())
	{
		DynamicsSolverPtr = MakeShared<FRigDynamicsSolver>(TEXT("DynamicsSolver"));
	}
	return DynamicsSolverPtr.Get();
}

