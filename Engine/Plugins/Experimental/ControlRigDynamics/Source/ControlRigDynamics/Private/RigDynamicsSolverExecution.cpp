// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsSolverExecution.h"
#include "RigDynamicsSolver.h"

#include "RigDynamicsHelpers.h"

#include "ControlRig.h"
#include "Rigs/RigHierarchyController.h"
#include "Units/RigUnitContext.h"
#include "Units/Execution/RigUnit_PrepareForExecution.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigDynamicsSolverExecution)

TAutoConsoleVariable<int32> CVarControlRigDynamicsAllowVisualization(
	TEXT("ControlRig.Dynamics.AllowVisualization"),
#if WITH_EDITOR
	1,
#elif UE_BUILD_SHIPPING || UE_BUILD_TEST
	0,
#else
	1,
#endif
	TEXT("Master allow-switch for control rig dynamics visualization. Set to 0 to suppress all debug drawing globally (e.g. for profiling). Defaults to 1 in editor and dev builds, 0 in shipping/test."));

TAutoConsoleVariable<int> CVarControlRigDynamicsEnableStepSolver(
	TEXT("ControlRig.Dynamics.EnableStepSolver"), 1,
	TEXT("Setting to zero disables stepping the solver"));

//======================================================================================================================
// Macros to reduce boilerplate for the Get/Set solver property nodes. Note that these can't
// be used in the header because UHT scans raw source before C++ preprocessing, so it would
// never see the USTRUCT/UPROPERTY/RIGVM_METHOD markers inside a macro expansion.
//======================================================================================================================

#define IMPLEMENT_SET_DYNAMICS_SOLVER(PropName)                                   \
FRigUnit_HierarchySetDynamicsSolver##PropName##_Execute()                         \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (FRigDynamicsSolverComponent* Component =                                  \
		GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))         \
	{                                                                             \
		Component->PropName = PropName;                                           \
	}                                                                             \
}

#define IMPLEMENT_GET_DYNAMICS_SOLVER(PropName)                                   \
FRigUnit_HierarchyGetDynamicsSolver##PropName##_Execute()                         \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (const FRigDynamicsSolverComponent* Component =                            \
		GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))         \
	{                                                                             \
		PropName = Component->PropName;                                           \
	}                                                                             \
}

#define IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(PropName)                           \
FRigUnit_HierarchySetDynamicsSolver##PropName##_Execute()                         \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (FRigDynamicsSolverComponent* Component =                                  \
		GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))         \
	{                                                                             \
		Component->Settings.PropName = PropName;                                  \
	}                                                                             \
}

#define IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(PropName)                           \
FRigUnit_HierarchyGetDynamicsSolver##PropName##_Execute()                         \
{                                                                                 \
	if (!ExecuteContext.Hierarchy) { return; }                                    \
	if (const FRigDynamicsSolverComponent* Component =                            \
		GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))         \
	{                                                                             \
		PropName = Component->Settings.PropName;                                  \
	}                                                                             \
}

//======================================================================================================================
// Deprecated entry-point. Cooked assets that pre-date the SpaceMotion / TeleportDetection split
// reference this struct and its legacy SimulationSpaceSettings + DragSettings pins. The legacy
// data is converted on the fly into the new component members so existing graphs continue to
// work; users get a one-click upgrade via GetUpgradeInfo below.
FRigUnit_SpawnDynamicsSolver_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsSolver can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsSolverComponentKey = Controller->AddComponent(
			FRigDynamicsSolverComponent::StaticStruct(), SolverComponentName, Owner);
		if (DynamicsSolverComponentKey.IsValid())
		{
			if (FRigDynamicsSolverComponent* Component =
				GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Component->Settings = Settings;
				ConvertLegacyDynamicsSimulationSpaceSettings(
					SimulationSpaceSettings, Component->SpaceMotion, Component->TeleportDetection);
				Component->SpaceMotion.Drag = DragSettings;
			}
		}
	}
}

//======================================================================================================================
FRigVMStructUpgradeInfo FRigUnit_SpawnDynamicsSolver::GetUpgradeInfo() const
{
	FRigUnit_SpawnDynamicsSolver1 NewNode;
	NewNode.Owner = Owner;
	NewNode.SolverComponentName = SolverComponentName;
	NewNode.Settings = Settings;
	ConvertLegacyDynamicsSimulationSpaceSettings(
		SimulationSpaceSettings, NewNode.SpaceMotion, NewNode.TeleportDetection);
	NewNode.SpaceMotion.Drag = DragSettings;

	FRigVMStructUpgradeInfo Info(*this, NewNode);

	// Sub-pin connections from the old monolithic SimulationSpaceSettings pin follow into the new
	// SpaceMotion / TeleportDetection split (drag-related sub-pins on SimulationSpaceSettings did
	// not exist - drag fields lived on the separate DragSettings pin, remapped further down).
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.SpaceMovementAmount"),                    TEXT("SpaceMotion.InertialForces.Amount"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.VelocityScaleZ"),                         TEXT("SpaceMotion.VerticalMotionScale"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampLinearVelocity"),                   TEXT("SpaceMotion.bClampLinearVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxLinearVelocity"),                      TEXT("SpaceMotion.MaxLinearVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampAngularVelocity"),                  TEXT("SpaceMotion.bClampAngularVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxAngularVelocity"),                     TEXT("SpaceMotion.MaxAngularVelocity"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampLinearAcceleration"),               TEXT("SpaceMotion.bClampLinearAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxLinearAcceleration"),                  TEXT("SpaceMotion.MaxLinearAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bClampAngularAcceleration"),              TEXT("SpaceMotion.bClampAngularAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.MaxAngularAcceleration"),                 TEXT("SpaceMotion.MaxAngularAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bTeleportFromLinearAcceleration"),        TEXT("TeleportDetection.bFromLinearAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.LinearAccelerationThresholdForTeleport"), TEXT("TeleportDetection.LinearAccelerationThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bTeleportFromAngularAcceleration"),       TEXT("TeleportDetection.bFromAngularAcceleration"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.AngularAccelerationThresholdForTeleport"),TEXT("TeleportDetection.AngularAccelerationThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bTeleportFromPositionChange"),            TEXT("TeleportDetection.bFromPositionChange"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.PositionChangeThresholdForTeleport"),     TEXT("TeleportDetection.PositionChangeThreshold"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.bTeleportFromOrientationChange"),         TEXT("TeleportDetection.bFromOrientationChange"));
	Info.AddRemappedPin(TEXT("SimulationSpaceSettings.OrientationChangeThresholdForTeleport"),  TEXT("TeleportDetection.OrientationChangeThreshold"));

	// DragSettings sub-pins move into the new SpaceMotion.Drag nested sub-pin.
	Info.AddRemappedPin(TEXT("DragSettings.LinearDragMultiplier"),       TEXT("SpaceMotion.Drag.LinearDragMultiplier"));
	Info.AddRemappedPin(TEXT("DragSettings.AngularDragMultiplier"),      TEXT("SpaceMotion.Drag.AngularDragMultiplier"));
	Info.AddRemappedPin(TEXT("DragSettings.ExternalLinearVelocity"),     TEXT("SpaceMotion.Drag.ExternalLinearVelocity"));
	Info.AddRemappedPin(TEXT("DragSettings.ExternalAngularVelocity"),    TEXT("SpaceMotion.Drag.ExternalAngularVelocity"));
	Info.AddRemappedPin(TEXT("DragSettings.ExternalTurbulenceVelocity"), TEXT("SpaceMotion.Drag.ExternalTurbulenceVelocity"));

	return Info;
}

//======================================================================================================================
FRigUnit_SpawnDynamicsSolver1_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (ExecuteContext.GetEventName() != FRigUnit_PrepareForExecution::EventName)
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("SpawnDynamicsSolver can only be used during Setup"));
		return;
	}

	if (URigHierarchyController* Controller = ExecuteContext.Hierarchy->GetController())
	{
		FRigHierarchyControllerInstructionBracket InstructionBracket(
			Controller, ExecuteContext.GetInstructionIndex());
		DynamicsSolverComponentKey = Controller->AddComponent(
			FRigDynamicsSolverComponent::StaticStruct(), SolverComponentName, Owner);
		if (DynamicsSolverComponentKey.IsValid())
		{
			if (FRigDynamicsSolverComponent* Component =
				GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
			{
				Component->Settings = Settings;
				Component->SpaceMotion = SpaceMotion;
				Component->TeleportDetection = TeleportDetection;
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsSolverExists_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	bExists = GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey) != nullptr;
}

//======================================================================================================================
FRigUnit_InstantiateDynamics_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	SCOPED_NAMED_EVENT(RigDynamicsSolver_Instantiate, FColor::Yellow);

	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	const FRigDynamicsSolverComponent* SolverComponent = GetSolver(Hierarchy, DynamicsSolverComponentKey);

	if (SolverComponent)
	{
		if (FRigDynamicsSolver* Solver = SolverComponent->GetDynamicsSolver())
		{
			Solver->Instantiate(ExecuteContext, Hierarchy, *SolverComponent);
		}
	}
}

//======================================================================================================================
FRigUnit_StepDynamicsSolver_Execute()
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_RIGUNIT();
	SCOPED_NAMED_EVENT(RigDynamicsSolver_StepSimulation, FColor::Yellow);

	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (CVarControlRigDynamicsEnableStepSolver.GetValueOnAnyThread() <= 0)
	{
		return;
	}
	URigHierarchy& Hierarchy = *ExecuteContext.Hierarchy;

	const FRigDynamicsSolverComponent* SolverComponent = GetSolver(Hierarchy, CachedSolverComponent);
		
	if (!SolverComponent || (SolverComponent->GetKey() != DynamicsSolverComponentKey))
	{
		CachedSolverComponent = FCachedRigComponent(DynamicsSolverComponentKey, &Hierarchy, true);
		SolverComponent = GetSolver(Hierarchy, CachedSolverComponent);
	}

	if (SolverComponent)
	{
		if (UControlRig* ControlRig = Cast<UControlRig>(Hierarchy.GetOuter()))
		{
			// We only set this flag once we start executing, because that's when it matters.
			// Control rig is responsible for clearing it when the rig is constructed.
			ControlRig->SetContainsSimulation();

			if (FRigDynamicsSolver* Solver = SolverComponent->GetDynamicsSolver())
			{
				const USceneComponent* OwningSceneComponent = ControlRig->GetOwningSceneComponent();
				const AActor* OwningActorPtr = OwningSceneComponent ? OwningSceneComponent->GetOwner() : nullptr;

				Solver->StepSimulation(
					ExecuteContext, Hierarchy, *SolverComponent, OwningActorPtr,
					DeltaTimeOverride, SimulationSpaceDeltaTimeOverride, Alpha, bTrackVelocitiesDuringPassThrough);

				if (Alpha > 0.0f
					&& CVarControlRigDynamicsAllowVisualization.GetValueOnAnyThread() != 0
					&& RigDynamicsShouldVisualize(VisualizationSettings))
				{
					Solver->Draw(
						ExecuteContext.GetDrawInterface(), Hierarchy,
						ExecuteContext.GetWorld(), VisualizationSettings);
				}
			}
		}
	}
}

//======================================================================================================================
FRigUnit_GetDynamicsSolverData_Execute()
{
	if (!ExecuteContext.Hierarchy)
	{
		return;
	}
	if (const FRigDynamicsSolverComponent* SolverComponent = 
		GetSolver(*ExecuteContext.Hierarchy, DynamicsSolverComponentKey))
	{
		if (const FRigDynamicsSolver* Solver = SolverComponent->GetDynamicsSolver())
		{
			const FRigDynamicsSimulationSpaceState& State = Solver->GetSimulationSpaceState();
			SimulationSpaceLinearVelocity = State.GetLinearVelocity();
			// State stores angular values in rad/s and rad/s/s; expose to users in deg/s and deg/s/s
			// to match the user-facing authoring conventions elsewhere in the plugin.
			SimulationSpaceAngularVelocity = FMath::RadiansToDegrees(State.GetAngularVelocity());
			SimulationSpaceLinearAcceleration = State.GetLinearAcceleration();
			SimulationSpaceAngularAcceleration = FMath::RadiansToDegrees(State.GetAngularAcceleration());
			bTeleportDetected = State.WasTeleportDetectedInLastUpdate();
			bKinematicSpeedResetTriggered = Solver->WasKinematicSpeedResetInLastStep();
			bPositionResetTriggered = Solver->WasPositionResetInLastStep();
			bEvaluationIntervalResetTriggered = Solver->WasEvaluationIntervalResetInLastStep();
		}
	}
}

//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER(Settings)
IMPLEMENT_GET_DYNAMICS_SOLVER(Settings)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER(SpaceMotion)
IMPLEMENT_GET_DYNAMICS_SOLVER(SpaceMotion)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER(TeleportDetection)
IMPLEMENT_GET_DYNAMICS_SOLVER(TeleportDetection)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(Gravity)
IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(Gravity)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(MaxTimeStep)
IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(MaxTimeStep)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(MaxNumSteps)
IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(MaxNumSteps)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(NumIterations)
IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(NumIterations)
//======================================================================================================================
IMPLEMENT_SET_DYNAMICS_SOLVER_SETTING(NumConstraintSubIterations)
IMPLEMENT_GET_DYNAMICS_SOLVER_SETTING(NumConstraintSubIterations)
