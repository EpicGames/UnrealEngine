// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigDynamicsSolver.h"

#include "Engine/Engine.h"

#include "Materials/MaterialInstanceDynamic.h"

#include "Math/Color.h"

TAutoConsoleVariable<int> CVarControlRigDynamicsShowParticlesOverride(
	TEXT("ControlRig.Dynamics.ShowParticlesOverride"), -1,
	TEXT("Whether to draw particles (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowCollidersOverride(
	TEXT("ControlRig.Dynamics.ShowCollidersOverride"), -1,
	TEXT("Whether to draw colliders (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowConfinersOverride(
	TEXT("ControlRig.Dynamics.ShowConfinersOverride"), -1,
	TEXT("Whether to draw confiners (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowSkeletalConstraintsOverride(
	TEXT("ControlRig.Dynamics.ShowSkeletalConstraintsOverride"), -1,
	TEXT("Whether to draw skeletal distance constraints (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowHardConstraintsOverride(
	TEXT("ControlRig.Dynamics.ShowHardConstraintsOverride"), -1,
	TEXT("Whether to draw hard distance constraints (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowSoftConstraintsOverride(
	TEXT("ControlRig.Dynamics.ShowSoftConstraintsOverride"), -1,
	TEXT("Whether to draw soft distance constraints (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowAngleLimitsOverride(
	TEXT("ControlRig.Dynamics.ShowAngleLimitsOverride"), -1,
	TEXT("Whether to draw per-particle angle limits (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowConeLimitsOverride(
	TEXT("ControlRig.Dynamics.ShowConeLimitsOverride"), -1,
	TEXT("Whether to draw cone limits (requires visualization to be enabled). -1 uses the visualization setting, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<int> CVarControlRigDynamicsShowForceFieldsOverride(
	TEXT("ControlRig.Dynamics.ShowForceFieldsOverride"), -1,
	TEXT("Whether per-instance force-field debug drawing is enabled (requires visualization to be enabled). -1 uses the field node's bDrawDebug pin, 0 forces drawing to be disabled, 1 forces it to be enabled."));

TAutoConsoleVariable<float> CVarControlRigDynamicsForceFieldDebugScaleOverride(
	TEXT("ControlRig.Dynamics.ForceFieldDebugScaleOverride"), -1.0f,
	TEXT("Override on the field node's DebugForceScale. Negative uses the field node's pin value; non-negative forces this scale (0 hides arrows; larger values elongate them)."));

TAutoConsoleVariable<FString> CVarControlRigDynamicsParticleValueDisplayOverride(
	TEXT("ControlRig.Dynamics.ParticleValueDisplayOverride"), TEXT(""),
	TEXT("Override for the per-particle numeric overlay. Empty string uses the visualization setting; otherwise the name of an ERigDynamicsParticleValueDisplay value (None, Radius, Mass, GravityMultiplier, Strength, DampingRatio, ExtraDamping, Drag, TargetMode, AngleLimit, AngleLimitStrength). Invalid strings silently fall back to the setting."));

// Owned by RigDynamicsSolverExecution.cpp; re-declared here so the force-field draw helpers can
// gate themselves the same way the step node does.
extern TAutoConsoleVariable<int32> CVarControlRigDynamicsAllowVisualization;

//======================================================================================================================
inline bool GetControlRigDynamicsShow(const bool SettingsValue, const TAutoConsoleVariable<int>& CVar)
{
	const int32 ShowOverride = CVar.GetValueOnAnyThread();
	return ShowOverride < 0 ? SettingsValue : (ShowOverride != 0);
}

//======================================================================================================================
inline ERigDynamicsParticleValueDisplay GetControlRigDynamicsParticleValueDisplay(
	ERigDynamicsParticleValueDisplay SettingsValue, const TAutoConsoleVariable<FString>& CVar)
{
	const FString OverrideString = CVar.GetValueOnAnyThread();
	if (OverrideString.IsEmpty())
	{
		return SettingsValue;
	}
	const UEnum* EnumPtr = StaticEnum<ERigDynamicsParticleValueDisplay>();
	const int64 Value = EnumPtr ? EnumPtr->GetValueByNameString(OverrideString) : INDEX_NONE;
	return Value == INDEX_NONE ? SettingsValue : static_cast<ERigDynamicsParticleValueDisplay>(Value);
}

//======================================================================================================================
// True if any per-element draw would actually emit geometry, given the rig's bShow* flags AND the
// per-element Show*Override CVars (plus the ParticleValueDisplay enum override - the per-particle text
// overlay can be force-enabled via the override CVar with all bShow* off). 
bool RigDynamicsShouldVisualize(const FRigDynamicsVisualizationSettings& VisualizationSettings)
{
	return GetControlRigDynamicsShow(
		VisualizationSettings.bShowParticles, CVarControlRigDynamicsShowParticlesOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowColliders, CVarControlRigDynamicsShowCollidersOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowConfiners, CVarControlRigDynamicsShowConfinersOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowSkeletalConstraints, CVarControlRigDynamicsShowSkeletalConstraintsOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowHardConstraints, CVarControlRigDynamicsShowHardConstraintsOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowSoftConstraints, CVarControlRigDynamicsShowSoftConstraintsOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowAngleLimits, CVarControlRigDynamicsShowAngleLimitsOverride)
		|| GetControlRigDynamicsShow(
			VisualizationSettings.bShowConeLimits, CVarControlRigDynamicsShowConeLimitsOverride)
		|| GetControlRigDynamicsParticleValueDisplay(
			VisualizationSettings.ParticleValueDisplay, CVarControlRigDynamicsParticleValueDisplayOverride)
			!= ERigDynamicsParticleValueDisplay::None;
}

//======================================================================================================================
bool ShouldDrawForceFieldDebug(const bool SettingsValue)
{
	return GetControlRigDynamicsShow(SettingsValue, CVarControlRigDynamicsShowForceFieldsOverride);
}

//======================================================================================================================
float GetForceFieldDebugScale(const float SettingsValue)
{
	const float Override = CVarControlRigDynamicsForceFieldDebugScaleOverride.GetValueOnAnyThread();
	return Override < 0.0f ? SettingsValue : Override;
}

//======================================================================================================================
namespace RigDynamicsSolverDraw
{
	const FLinearColor SimulatedParticleColor = FColor::Yellow;
	const FLinearColor KinematicParticleColor = FColor::Blue;
	const FLinearColor ColliderColor          = FColor::Red;
	const FLinearColor ConfinerColor          = FColor::Cyan;
	const FLinearColor ConstraintColor        = FColor::White;  // target/satisfied portion
	const FLinearColor StretchColor           = FColor::Green;  // over-extended portion
	const FLinearColor CompressColor          = FColor::Red;    // "needs to grow" overhang
	const FLinearColor ForceFieldColor        = FLinearColor(1.0f, 0.0f, 1.0f); // magenta

	// Angle-limit and cone-limit cones share a primitive but use different colour families so they
	// can be told apart at a glance when both are enabled. The "exceeded" variants are red with the
	// same amount of the inside-tint channel mixed in, so each cone keeps its identity even
	// after the limit is broken.
	const FLinearColor AngleLimitInsideColor  = FLinearColor(0.7f, 1.0f, 0.7f);  // pale green
	const FLinearColor AngleLimitOutsideColor = FLinearColor(1.0f, 0.3f, 0.0f);  // red + green
	const FLinearColor ConeLimitInsideColor   = FLinearColor(0.7f, 0.7f, 1.0f);  // pale blue
	const FLinearColor ConeLimitOutsideColor  = FLinearColor(1.0f, 0.0f, 0.3f);  // red + blue

	static float GetParticlePropertyValue(
		const FRigDynamicsParticleProperties& Props, ERigDynamicsParticleValueDisplay Which)
	{
		switch (Which)
		{
		case ERigDynamicsParticleValueDisplay::Radius:             return Props.Radius;
		case ERigDynamicsParticleValueDisplay::Mass:               return Props.Mass;
		case ERigDynamicsParticleValueDisplay::GravityMultiplier:  return Props.GravityMultiplier;
		case ERigDynamicsParticleValueDisplay::Strength:           return Props.Strength;
		case ERigDynamicsParticleValueDisplay::DampingRatio:       return Props.DampingRatio;
		case ERigDynamicsParticleValueDisplay::ExtraDamping:       return Props.ExtraDamping;
		case ERigDynamicsParticleValueDisplay::Damping:            return Props.Damping;
		case ERigDynamicsParticleValueDisplay::TargetMode:         return Props.TargetMode;
		case ERigDynamicsParticleValueDisplay::AngleLimit:         return Props.AngleLimit;
		case ERigDynamicsParticleValueDisplay::AngleLimitStrength: return Props.AngleLimitStrength;
		default:                                                   return 0.0f;
		}
	}

	//==================================================================================================================
	// Draws a distance constraint between two particles, showing the target distance:
	//   Current > Target (stretched): white central segment of length Target, green extensions
	//                                 out to the particles on each side.
	//   Current < Target (compressed): white line between the particles, red extensions beyond
	//                                  each particle so total end-to-end length equals Target.
	//   |Current - Target| tiny, or Target/Current degenerate: single white line.
	static void DrawDistanceConstraintLine(
		FRigVMDrawInterface* DI, const FVector& PosA, const FVector& PosB, float TargetDistance, float Thickness)
	{
		const FVector Delta = PosB - PosA;
		const float Current = Delta.Size();

		if (TargetDistance < KINDA_SMALL_NUMBER
			|| Current < KINDA_SMALL_NUMBER
			|| FMath::Abs(Current - TargetDistance) < KINDA_SMALL_NUMBER)
		{
			DI->DrawLine(FTransform::Identity, PosA, PosB, ConstraintColor, Thickness);
			return;
		}

		const FVector Dir = Delta / Current;

		if (Current > TargetDistance)
		{
			const FVector Mid = (PosA + PosB) * 0.5f;
			const FVector HalfTargetOffset = Dir * (TargetDistance * 0.5f);
			const FVector W1 = Mid - HalfTargetOffset;
			const FVector W2 = Mid + HalfTargetOffset;
			DI->DrawLine(FTransform::Identity, PosA, W1, StretchColor, Thickness);
			DI->DrawLine(FTransform::Identity, W1, W2, ConstraintColor, Thickness);
			DI->DrawLine(FTransform::Identity, W2, PosB, StretchColor, Thickness);
		}
		else
		{
			const FVector HalfOverhang = Dir * ((TargetDistance - Current) * 0.5f);
			DI->DrawLine(FTransform::Identity, PosA - HalfOverhang, PosA, CompressColor, Thickness);
			DI->DrawLine(FTransform::Identity, PosA, PosB, ConstraintColor, Thickness);
			DI->DrawLine(FTransform::Identity, PosB, PosB + HalfOverhang, CompressColor,   Thickness);
		}
	}

	//==================================================================================================================
	template<typename TInfo, typename TConstraint>
	static void DrawConstraintSet(
		FRigVMDrawInterface*                               DI,
		const RigParticleSimulation::FSimulationState&     SimulationState,
		const FRigDynamicsSimulationSpaceState&             SimulationSpaceState,
		const TArray<TInfo>&                               Infos,
		const TArray<TConstraint>&                         Constraints,
		const float                                        Thickness)
	{
		const int32 NumConstraints = Constraints.Num();
		for (int32 Index = 0; Index < NumConstraints; ++Index)
		{
			const TInfo& Info = Infos[Index];
			const FVector A = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
				SimulationState.Particles[Info.ParentIndex].Position);
			const FVector B = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
				SimulationState.Particles[Info.ChildIndex].Position);
			DrawDistanceConstraintLine(DI, A, B, Constraints[Index].TargetDistance, Thickness);
		}
	}

	//==================================================================================================================
	// Draws a wireframe cone representing an angle limit. Apex is at the parent, axis is along
	// TargetDir, and the cone's slant (apex-to-rim) equals the current bone length so the child
	// particle sits exactly on the rim when its direction is at the limit. Valid for all angles
	// from 0 to 180 degrees; the mouth slides back past the apex for angles > 90 (the "cone"
	// points backwards), which is visually ambiguous but well-defined. Drawn in InsideColor when
	// the current bone direction is within the limit, OutsideColor when it has swung outside.
	static void DrawAngleLimit(
		FRigVMDrawInterface* DI,
		const FVector&       ParentPos,
		const FVector&       ChildPos,
		const FVector&       TargetDir,
		const float          CosAngleLimit,
		const float          SinAngleLimit,
		const float          Thickness,
		const int32          NumSides,
		const FLinearColor&  InsideColor,
		const FLinearColor&  OutsideColor)
	{
		const FVector Bone = ChildPos - ParentPos;
		const float BoneLength = Bone.Size();
		if (BoneLength < KINDA_SMALL_NUMBER)
		{
			return;  // Colocated; no current direction to compare.
		}

		const FVector CurrentDir = Bone / BoneLength;
		const bool bInside = FVector::DotProduct(CurrentDir, TargetDir) >= CosAngleLimit;
		const FLinearColor& Color = bInside ? InsideColor : OutsideColor;

		// Orthonormal basis around TargetDir.
		const FVector UpGuess = (FMath::Abs(TargetDir.Z) < 0.9f)
			? FVector(0.0f, 0.0f, 1.0f) : FVector(1.0f, 0.0f, 0.0f);
		const FVector Perp1 = FVector::CrossProduct(TargetDir, UpGuess).GetSafeNormal();
		const FVector Perp2 = FVector::CrossProduct(TargetDir, Perp1);

		// Slant = BoneLength. Mouth centre is BoneLength*cos(α) along the axis (negative for α>90°),
		// mouth radius is BoneLength*sin(α). Well-defined for the full 0..180° range.
		const FVector MouthCenter = ParentPos + TargetDir * (BoneLength * CosAngleLimit);
		const float MouthRadius = BoneLength * SinAngleLimit;

		TArray<FVector, TInlineAllocator<32>> MouthPoints;
		MouthPoints.Reserve(NumSides);
		for (int32 I = 0; I < NumSides; ++I)
		{
			const float T = (2.0f * PI * I) / NumSides;
			const FVector Offset = (Perp1 * FMath::Cos(T) + Perp2 * FMath::Sin(T)) * MouthRadius;
			MouthPoints.Add(MouthCenter + Offset);
		}

		// Perimeter ring.
		for (int32 I = 0; I < NumSides; ++I)
		{
			const int32 J = (I + 1) % NumSides;
			DI->DrawLine(FTransform::Identity, MouthPoints[I], MouthPoints[J], Color, Thickness);
		}

		// Radial lines from apex. Cap at 8 to avoid a spiderweb on fine details.
		const int32 NumRadialLines = FMath::Min(NumSides, 8);
		for (int32 I = 0; I < NumRadialLines; ++I)
		{
			const int32 Idx = (I * NumSides) / NumRadialLines;
			DI->DrawLine(FTransform::Identity, ParentPos, MouthPoints[Idx], Color, Thickness);
		}
	}
}

//======================================================================================================================
void FRigDynamicsSolver::Draw(
	FRigVMDrawInterface*                     DI,
	const URigHierarchy&                     Hierarchy,
	const UWorld*                            DebugWorld,
	const FRigDynamicsVisualizationSettings& VisualizationSettings) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigDynamics_Draw);

	if (!DI)
	{
		return;
	}

	const bool bShowParticles = GetControlRigDynamicsShow(
		VisualizationSettings.bShowParticles, CVarControlRigDynamicsShowParticlesOverride);
	const bool bShowColliders = GetControlRigDynamicsShow(
		VisualizationSettings.bShowColliders, CVarControlRigDynamicsShowCollidersOverride);
	const bool bShowConfiners = GetControlRigDynamicsShow(
		VisualizationSettings.bShowConfiners, CVarControlRigDynamicsShowConfinersOverride);
	const bool bShowSkeletalConstraints = GetControlRigDynamicsShow(
		VisualizationSettings.bShowSkeletalConstraints, CVarControlRigDynamicsShowSkeletalConstraintsOverride);
	const bool bShowHardConstraints = GetControlRigDynamicsShow(
		VisualizationSettings.bShowHardConstraints, CVarControlRigDynamicsShowHardConstraintsOverride);
	const bool bShowSoftConstraints = GetControlRigDynamicsShow(
		VisualizationSettings.bShowSoftConstraints, CVarControlRigDynamicsShowSoftConstraintsOverride);
	const bool bShowAngleLimits = GetControlRigDynamicsShow(
		VisualizationSettings.bShowAngleLimits, CVarControlRigDynamicsShowAngleLimitsOverride);
	const bool bShowConeLimits = GetControlRigDynamicsShow(
		VisualizationSettings.bShowConeLimits, CVarControlRigDynamicsShowConeLimitsOverride);
	const ERigDynamicsParticleValueDisplay EffectiveValueDisplay =
		GetControlRigDynamicsParticleValueDisplay(
			VisualizationSettings.ParticleValueDisplay, CVarControlRigDynamicsParticleValueDisplayOverride);

	// Draw colliders at their current positions, converting from simulation space to component space
	if (bShowColliders)
	{
		for (const RigParticleSimulation::FShapeCollection& Collider : SimulationState.Colliders)
		{
			for (const RigParticleSimulation::FPlaneShape& PlaneShape : Collider.PlaneShapes)
			{
				if (GEngine && GEngine->ConstraintLimitMaterialPrismatic)
				{
					DI->DrawPlane(
						SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(PlaneShape.TM * Collider.TM),
						FVector2D(PlaneShape.Extents * 0.5f), RigDynamicsSolverDraw::ColliderColor, true,
						RigDynamicsSolverDraw::ColliderColor,
						GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy());
				}
			}
			for (const RigParticleSimulation::FBoxShape& BoxShape : Collider.BoxShapes)
			{
				DI->DrawBox(
					SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(BoxShape.TM * Collider.TM),
					FTransform(FQuat::Identity, FVector::ZeroVector, FVector(BoxShape.Extents)),
					RigDynamicsSolverDraw::ColliderColor, VisualizationSettings.LineThickness);
			}
			for (const RigParticleSimulation::FCapsuleShape& CapsuleShape : Collider.CapsuleShapes)
			{
				const FTransform ShapeTM =
					SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(CapsuleShape.TM * Collider.TM);
				if (CapsuleShape.Length < KINDA_SMALL_NUMBER)
				{
					DI->DrawSphere(ShapeTM, FTransform(), CapsuleShape.Radius,
						RigDynamicsSolverDraw::ColliderColor, VisualizationSettings.LineThickness,
						VisualizationSettings.ShapeDetail);
				}
				else
				{
					DI->DrawCapsule(ShapeTM, FTransform(), CapsuleShape.Radius, CapsuleShape.Length,
						RigDynamicsSolverDraw::ColliderColor, VisualizationSettings.LineThickness,
						VisualizationSettings.ShapeDetail);
				}
			}
		}
	}

	// Draw confiners. Shape layout mirrors colliders; the distinct colour marks them as
	// containment volumes (particles are pushed inside them rather than outside).
	if (bShowConfiners)
	{
		for (const RigParticleSimulation::FShapeCollection& Confiner : SimulationState.Confiners)
		{
			for (const RigParticleSimulation::FPlaneShape& PlaneShape : Confiner.PlaneShapes)
			{
				if (GEngine && GEngine->ConstraintLimitMaterialPrismatic)
				{
					DI->DrawPlane(
						SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(PlaneShape.TM * Confiner.TM),
						FVector2D(PlaneShape.Extents * 0.5f), RigDynamicsSolverDraw::ConfinerColor,
						true, RigDynamicsSolverDraw::ConfinerColor, 
						GEngine->ConstraintLimitMaterialPrismatic->GetRenderProxy());
				}
			}
			for (const RigParticleSimulation::FBoxShape& BoxShape : Confiner.BoxShapes)
			{
				DI->DrawBox(
					SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(BoxShape.TM * Confiner.TM),
					FTransform(FQuat::Identity, FVector::ZeroVector, FVector(BoxShape.Extents)),
					RigDynamicsSolverDraw::ConfinerColor, VisualizationSettings.LineThickness);
			}
			for (const RigParticleSimulation::FCapsuleShape& CapsuleShape : Confiner.CapsuleShapes)
			{
				const FTransform ShapeTM =
					SimulationSpaceState.ConvertSimSpaceTransformToComponentSpace(CapsuleShape.TM * Confiner.TM);
				if (CapsuleShape.Length < KINDA_SMALL_NUMBER)
				{
					DI->DrawSphere(ShapeTM, FTransform(), CapsuleShape.Radius,
						RigDynamicsSolverDraw::ConfinerColor, VisualizationSettings.LineThickness,
						VisualizationSettings.ShapeDetail);
				}
				else
				{
					DI->DrawCapsule(ShapeTM, FTransform(), CapsuleShape.Radius, CapsuleShape.Length,
						RigDynamicsSolverDraw::ConfinerColor, VisualizationSettings.LineThickness,
						VisualizationSettings.ShapeDetail);
				}
			}
		}
	}

	// Distance constraints (skeletal/hard/soft).
	if (bShowSkeletalConstraints)
	{
		RigDynamicsSolverDraw::DrawConstraintSet(
			DI, SimulationState, SimulationSpaceState, SimulationState.SkeletalConstraintInfos,
			SimulationState.SkeletalConstraints, VisualizationSettings.LineThickness);
	}
	if (bShowHardConstraints)
	{
		RigDynamicsSolverDraw::DrawConstraintSet(
			DI, SimulationState, SimulationSpaceState, SimulationState.HardDistanceConstraintInfos,
			SimulationState.HardDistanceConstraints, VisualizationSettings.LineThickness);
	}
	if (bShowSoftConstraints)
	{
		RigDynamicsSolverDraw::DrawConstraintSet(
			DI, SimulationState, SimulationSpaceState, SimulationState.SoftDistanceConstraintInfos,
			SimulationState.SoftDistanceConstraints, VisualizationSettings.LineThickness);
	}

	if (bShowAngleLimits)
	{
		const int32 NumParticles = SimulationState.Particles.Num();
		for (int32 Index = 0; Index < NumParticles; ++Index)
		{
			const RigParticleSimulation::FParticleInfo& Info = SimulationState.ParticleInfos[Index];
			if (Info.ParentParticleIndex == INDEX_NONE)
			{
				continue;
			}

			const RigParticleSimulation::FParticleTarget& Target = SimulationState.ParticleTargets[Index];
			if (Target.AngleLimitCompliance <= 0.0f)
			{
				continue;
			}

			const RigParticleSimulation::FSimVector& SimParent =
				SimulationState.Particles[Info.ParentParticleIndex].Position;
			const FVector ParentPos = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(SimParent);
			const FVector ChildPos  = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
				SimulationState.Particles[Index].Position);

			// Sim->component is a rigid transform, so offset the parent by the unit target direction,
			// convert, and take the delta to get the component-space direction.
			const FVector ParentPlusDirCS = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
				SimParent + Target.TargetDirectionFromParent);
			const FVector TargetDirCS = (ParentPlusDirCS - ParentPos).GetSafeNormal();

			RigDynamicsSolverDraw::DrawAngleLimit(
				DI, ParentPos, ChildPos, TargetDirCS, Target.CosAngleLimit, Target.SinAngleLimit,
				VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail,
				RigDynamicsSolverDraw::AngleLimitInsideColor, RigDynamicsSolverDraw::AngleLimitOutsideColor);
		}
	}

	// Cone limits (triple-particle grandparent/parent/child angular constraint). Apex at the parent,
	// axis along the incoming bone direction (grandparent->parent), slant = current parent->child
	// length so the child sits exactly on the rim when its direction is at the limit.
	if (bShowConeLimits)
	{
		const int32 NumConeLimits = SimulationState.ConeLimits.Num();
		for (int32 Index = 0; Index < NumConeLimits; ++Index)
		{
			const RigParticleSimulation::FConeLimitInfo& Info = SimulationState.ConeLimitInfos[Index];
			const RigParticleSimulation::FConeLimit&     ConeLimit = SimulationState.ConeLimits[Index];

			const RigParticleSimulation::FSimVector& SimGrandparent =
				SimulationState.Particles[Info.GrandparentIndex].Position;
			const RigParticleSimulation::FSimVector& SimParent =
				SimulationState.Particles[Info.ParentIndex].Position;

			const FVector GrandparentPos =
				SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(SimGrandparent);
			const FVector ParentPos =
				SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(SimParent);
			const FVector ChildPos = SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(
				SimulationState.Particles[Info.ChildIndex].Position);

			const FVector IncomingBone = ParentPos - GrandparentPos;
			const float   IncomingLen  = IncomingBone.Size();
			if (IncomingLen < KINDA_SMALL_NUMBER)
			{
				continue;  // Grandparent and parent colocated; no axis to draw against.
			}
			const FVector TargetDirCS = IncomingBone / IncomingLen;

			const float CosAngle = FMath::Cos(ConeLimit.Angle);
			const float SinAngle = FMath::Sin(ConeLimit.Angle);

			RigDynamicsSolverDraw::DrawAngleLimit(
				DI, ParentPos, ChildPos, TargetDirCS, CosAngle, SinAngle,
				VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail,
				RigDynamicsSolverDraw::ConeLimitInsideColor, RigDynamicsSolverDraw::ConeLimitOutsideColor);
		}
	}

	if (bShowParticles)
	{
		const int32 NumParticles = SimulationState.Particles.Num();
		for (int32 Index = 0; Index != NumParticles; ++Index)
		{
			const RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];
			const RigParticleSimulation::FParticleCollider& ParticleCollider = SimulationState.ParticleColliders[Index];

			FTransform ParticleComponentTM = Hierarchy.GetGlobalTransform(
				ParticleOwnerComponents[Index].GetElementKey());
			ParticleComponentTM.SetTranslation(
				SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(Particle.Position));

			DI->DrawSphere(
				ParticleComponentTM, FTransform(), ParticleCollider.Radius, Particle.InvMass != 0.0f 
				? RigDynamicsSolverDraw::SimulatedParticleColor : RigDynamicsSolverDraw::KinematicParticleColor,
				VisualizationSettings.LineThickness, VisualizationSettings.ShapeDetail);
		}
	}

	if (EffectiveValueDisplay != ERigDynamicsParticleValueDisplay::None)
	{
		const int32 NumParticles = SimulationState.Particles.Num();
		for (int32 Index = 0; Index != NumParticles; ++Index)
		{
			const FRigDynamicsParticleComponent* ParticleComponent =
				GetParticle(Hierarchy, ParticleOwnerComponents[Index].GetComponentKey());
			if (!ParticleComponent)
			{
				continue;
			}

			const float Value = RigDynamicsSolverDraw::GetParticlePropertyValue(
				ParticleComponent->ParticleProperties, EffectiveValueDisplay);

			const RigParticleSimulation::FParticle& Particle = SimulationState.Particles[Index];
			const FVector ComponentSpacePos =
				SimulationSpaceState.ConvertSimSpacePositionToComponentSpace(Particle.Position);

			DI->DrawText(
				FTransform::Identity, ComponentSpacePos, FString::Printf(TEXT("%.2f"), Value), FLinearColor::White);
		}
	}
}

//======================================================================================================================
void DrawForceFieldEllipsoid(
	FRigVMDrawInterface* DI, const FTransform& Pose, const FVector& Radii,
	float Thickness, int32 Detail)
{
	if (!DI || CVarControlRigDynamicsAllowVisualization.GetValueOnAnyThread() == 0)
	{
		return;
	}

	const int32 NumSides = FMath::Max(Detail, 6);
	const float StepAngle = TWO_PI / static_cast<float>(NumSides);

	TArray<FVector, TInlineAllocator<32>> Points;
	Points.Reserve(NumSides + 1);

	// XY ring (Z = 0 in field-local space)
	Points.Reset();
	for (int32 i = 0; i <= NumSides; ++i)
	{
		const float Theta = StepAngle * i;
		const FVector LocalP(Radii.X * FMath::Cos(Theta), Radii.Y * FMath::Sin(Theta), 0.0);
		Points.Add(Pose.TransformPositionNoScale(LocalP));
	}
	DI->DrawLineStrip(FTransform::Identity, Points, RigDynamicsSolverDraw::ForceFieldColor, Thickness);

	// YZ ring (X = 0 in field-local space)
	Points.Reset();
	for (int32 i = 0; i <= NumSides; ++i)
	{
		const float Theta = StepAngle * i;
		const FVector LocalP(0.0, Radii.Y * FMath::Cos(Theta), Radii.Z * FMath::Sin(Theta));
		Points.Add(Pose.TransformPositionNoScale(LocalP));
	}
	DI->DrawLineStrip(FTransform::Identity, Points, RigDynamicsSolverDraw::ForceFieldColor, Thickness);

	// XZ ring (Y = 0 in field-local space)
	Points.Reset();
	for (int32 i = 0; i <= NumSides; ++i)
	{
		const float Theta = StepAngle * i;
		const FVector LocalP(Radii.X * FMath::Cos(Theta), 0.0, Radii.Z * FMath::Sin(Theta));
		Points.Add(Pose.TransformPositionNoScale(LocalP));
	}
	DI->DrawLineStrip(FTransform::Identity, Points, RigDynamicsSolverDraw::ForceFieldColor, Thickness);
}

//======================================================================================================================
void DrawForceFieldArrow(
	FRigVMDrawInterface* DI, const FVector& Origin, const FVector& Vector,
	float Thickness)
{
	if (!DI || CVarControlRigDynamicsAllowVisualization.GetValueOnAnyThread() == 0)
	{
		return;
	}

	const float Length = static_cast<float>(Vector.Size());
	if (Length < KINDA_SMALL_NUMBER)
	{
		return;
	}

	// Pick a perpendicular axis using the same up-guess pattern as DrawAngleLimit. Side controls
	// the barb placement on FRigVMDrawInterface::DrawArrow: the barbs sit Side.Size() back along
	// Direction from the tip, offset by ±Side perpendicular to the shaft.
	const FVector Dir = Vector / Length;
	const FVector UpGuess = (FMath::Abs(Dir.Z) < 0.9f) ? FVector(0.0, 0.0, 1.0) : FVector(1.0, 0.0, 0.0);
	const FVector PerpUnit = FVector::CrossProduct(Dir, UpGuess).GetSafeNormal();
	const FVector Side = PerpUnit * (Length * 0.15f);

	const FTransform WorldOffset(FQuat::Identity, Origin);
	DI->DrawArrow(WorldOffset, Vector, Side, RigDynamicsSolverDraw::ForceFieldColor, Thickness);
}

