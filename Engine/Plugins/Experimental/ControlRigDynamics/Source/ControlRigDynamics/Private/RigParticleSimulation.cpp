// Copyright Epic Games, Inc. All Rights Reserved.

#include "RigParticleSimulation.h"

//======================================================================================================================
// This should be kept completely independent of control rig etc
//======================================================================================================================

namespace RigParticleSimulation
{

//======================================================================================================================
// Variant of FVector::GetSafeNormal that skips the "SquareSum == 1" early-out check. That check
// is useful when the caller is likely to be normalising unit-length vectors, but in our hot paths
// the input is (particle - parent) style differences that are rarely exactly unit length, so the
// extra compare is pure overhead.
static FORCEINLINE FSimVector GetSafeNormalFast(const FSimVector& V)
{
	const FSimScalar SquareSum = V.SizeSquared();
	return (SquareSum < UE_SMALL_NUMBER) ? FSimVector::ZeroVector : V * FMath::InvSqrt(SquareSum);
}

//======================================================================================================================
static void StepParticleDynamic(
	FParticle&                      Particle,
	const FParticleInfo&            Info,
	const float                     InDeltaTime,
	const float                     InDtRatio,
	const float                     InStepAlpha,
	const FSimVector&               InGravity,
	const FSimulationSpaceMotion&   SimulationSpaceMotion)
{
	check(Particle.InvMass != 0.0f);

	Particle.TargetPosition = FMath::Lerp(Info.PreviousTargetPosition, Info.TargetPosition, InStepAlpha);
	Particle.TargetVelocity = FMath::Lerp(Info.PreviousTargetVelocity, Info.TargetVelocity, InStepAlpha);

	// Time-corrected Verlet integration. Standard Verlet assumes constant dt:
	//   NewPos = 2*Pos - PrevPos + Accel*dt^2
	// which is equivalent to:
	//   NewPos = Pos + (Pos - PrevPos) + Accel*dt^2
	// The term (Pos - PrevPos) is the implicit velocity * dt_prev. When the timestep changes
	// (e.g. at frame boundaries due to variable framerate or substepping changes), we scale the
	// velocity term by (dt_current / dt_prev) to account for the different time interval.
	FSimVector Acceleration = InGravity + Info.ExternalForce * Particle.InvMass;

	// Non-inertial reference frame pseudo-forces. These are all mass-proportional (the
	// acceleration is the same regardless of particle mass, like gravity).
	const float InertialForceAmount = SimulationSpaceMotion.InertialForceAmount;
	if (InertialForceAmount > KINDA_SMALL_NUMBER)
	{
		const FSimVector R = Particle.Position;
		const FSimVector& W = SimulationSpaceMotion.AngularVelocity;

		FSimVector PseudoAccel = FSimVector::ZeroVector;

		if (SimulationSpaceMotion.bAccelerationsValid)
		{
			// Fictitious force from linear acceleration: -a_frame
			PseudoAccel -= SimulationSpaceMotion.LinearAcceleration * SimulationSpaceMotion.LinearEulerAmount;
			// Euler force from angular acceleration: -alpha x r
			PseudoAccel -= FSimVector::CrossProduct(SimulationSpaceMotion.AngularAcceleration, R) * SimulationSpaceMotion.AngularEulerAmount;
		}

		// Centrifugal: -w x (w x r)
		const FSimVector WxR = FSimVector::CrossProduct(W, R);
		PseudoAccel -= FSimVector::CrossProduct(W, WxR) * SimulationSpaceMotion.CentrifugalAmount;

		// Coriolis: -2(w x v), using the implicit Verlet velocity
		const FSimVector V = (Particle.Position - Particle.PrevPosition) / InDeltaTime;
		PseudoAccel -= (2.0f * SimulationSpaceMotion.CoriolisAmount) * FSimVector::CrossProduct(W, V);

		Acceleration += PseudoAccel * InertialForceAmount;
	}

	const FSimVector OldPosition = Particle.Position;
	Particle.Position += 
		(Particle.Position - Particle.PrevPosition) * InDtRatio + Acceleration * FMath::Square(InDeltaTime);
	Particle.PrevPosition = OldPosition;

	// Air/ether drag: relax the particle's velocity toward the local air/ether velocity in
	// simulation space. Per-particle Info.Damping and the global SimulationSpaceMotion.AdditionalDamping
	// sum into a single effective rate, and the per-particle bScaleDampingByInverseMass governs
	// both: when true, the effective rate is divided by mass so lighter particles damp faster;
	// when false, every particle has the same relaxation timescale regardless of mass. The global
	// AdditionalDamping deliberately follows the per-particle flag so both terms share one mass-scaling
	// choice - mixing mass-scaled with mass-independent within a single particle would be confusing.
	const float EffectiveDamping = Info.Damping + SimulationSpaceMotion.AdditionalDamping;
	if (EffectiveDamping > KINDA_SMALL_NUMBER)
	{
		const FSimVector R = Particle.Position;

		// Air/ether velocity is the sum of two contributions:
		// * Sim-space motion: world-at-rest as seen from the moving frame, scaled by the drag
		//   multipliers. InertialForceAmount deliberately does not apply here - it only gates
		//   the inertial pseudo-forces above.
		// * External air/ether velocity authored by the user (wind / vortex / turbulence), applied
		//   1:1 without scaling by the drag multipliers.
		const FSimVector WxR = FSimVector::CrossProduct(SimulationSpaceMotion.AngularVelocity, R);
		const FSimVector ExtWxR = FSimVector::CrossProduct(SimulationSpaceMotion.ExternalAngularVelocity, R);

		const FSimVector EtherVel =
			-SimulationSpaceMotion.LinearVelocity * SimulationSpaceMotion.LinearDragMultiplier
			- WxR * SimulationSpaceMotion.AngularDragMultiplier
			+ SimulationSpaceMotion.ExternalLinearVelocity
			+ ExtWxR;

		const float Exponent = Info.bScaleDampingByInverseMass
			? EffectiveDamping * Particle.InvMass * InDeltaTime : EffectiveDamping * InDeltaTime;
		const float DampingRate = 1.0f - FMath::Exp(-Exponent);

		const FSimVector CurrentVel = (Particle.Position - Particle.PrevPosition) / InDeltaTime;
		const FSimVector NewVel = FMath::Lerp(CurrentVel, EtherVel, DampingRate);
		Particle.PrevPosition = Particle.Position - NewVel * InDeltaTime;
	}
}

//======================================================================================================================
static void StepParticleKinematic(
	FParticle&           Particle,
	const FParticleInfo& Info,
	const float          InDeltaTime,
	const float          InStepAlpha)
{
	check(Particle.InvMass == 0.0f);

	Particle.TargetPosition = FMath::Lerp(Info.PreviousTargetPosition, Info.TargetPosition, InStepAlpha);
	Particle.TargetVelocity = FMath::Lerp(Info.PreviousTargetVelocity, Info.TargetVelocity, InStepAlpha);

	// Kinematic - just track the target position
	Particle.Position = Particle.TargetPosition;
	Particle.PrevPosition = Particle.Position - Particle.TargetVelocity * InDeltaTime;
}

//======================================================================================================================
static void StepParticleTarget(
	FParticleTarget& ParticleTarget, const float InStepAlpha,
	const FSimVector& InPrevTargetDirectionFromParent, const FSimVector& InTargetDirectionFromParent)
{
	// Lerp + normalize is a good approximation of Slerp for the small inter-substep angles
	const FSimVector InterpDir =
		FMath::Lerp(InPrevTargetDirectionFromParent, InTargetDirectionFromParent, InStepAlpha);
	const FSimScalar InterpLen = InterpDir.Size();
	ParticleTarget.TargetDirectionFromParent =
		(InterpLen > KINDA_SMALL_NUMBER) ? InterpDir * (1.0f / InterpLen) : FSimVector::ZeroVector;
	ParticleTarget.Lambda = FSimVector::ZeroVector;
}

//======================================================================================================================
static void StepSoftDistanceConstraint(FSoftDistanceConstraint& Constraint)
{
	Constraint.Lambda = 0.0f;
}

//======================================================================================================================
static void StepConeLimit(FConeLimit& ConeLimit)
{
	ConeLimit.Lambda = 0.0f;
}

//======================================================================================================================
static void StepShapeCollection(FShapeCollection& Collider, const float StepAlpha)
{
	Collider.TM.Blend(Collider.PreviousTM, Collider.TargetTM, StepAlpha);
}

//======================================================================================================================
static void SolveTargetConstraint(
	FParticle&        Particle,
	FParticleTarget&  ParticleTarget,
	FParticle*        ParentParticlePtr,
	const bool        bOnlyMoveChild,
	const float       DeltaTime,
	const float       InvDeltaTime)
{
	// Note that we should never be called if InvMass = 0 (the caller should be sensible!)
	check(Particle.InvMass > 0.0f);

	// Alpha blends between SimSpace (0) and directional (1). If there's no parent, force
	// SimSpace regardless of the setting.
	const float Alpha = ParentParticlePtr ? ParticleTarget.TargetMode : 0.0f;

	// SimSpace constraint error: drive child toward its absolute target position
	const FSimVector C_sim = Particle.Position - Particle.TargetPosition;

	// Directional constraint error: drive parent-child direction to match animation
	FSimVector C_dir = FSimVector::ZeroVector;
	if (Alpha > KINDA_SMALL_NUMBER && ParentParticlePtr)
	{
		const FSimVector Delta = Particle.Position - ParentParticlePtr->Position;
		const FSimScalar DeltaLen = Delta.Size();
		if (DeltaLen > KINDA_SMALL_NUMBER
			&& !ParticleTarget.TargetDirectionFromParent.IsNearlyZero(KINDA_SMALL_NUMBER))
		{
			C_dir = Delta - ParticleTarget.TargetDirectionFromParent * DeltaLen;
		}
	}

	// Blended constraint error
	const FSimVector C = FMath::Lerp(C_sim, C_dir, Alpha);

	// Bilateral mass: w_child + Alpha^2 * w_parent
	const float ParentInvMass = ParentParticlePtr ? ParentParticlePtr->InvMass : 0.0f;
	const float InvMassForDenom =
		bOnlyMoveChild ? Particle.InvMass : (Particle.InvMass + Alpha * Alpha * ParentInvMass);

	// Blended velocity: child velocity minus Alpha-scaled parent velocity
	const FSimVector ChildVel = (Particle.Position - Particle.PrevPosition) * InvDeltaTime;
	FSimVector CurrentVelocity = ChildVel;
	if (Alpha > KINDA_SMALL_NUMBER && ParentParticlePtr)
	{
		const FSimVector ParentVel =
			(ParentParticlePtr->Position - ParentParticlePtr->PrevPosition) * InvDeltaTime;
		CurrentVelocity -= Alpha * ParentVel;
	}

	FSimVector RefVelocity = Particle.TargetVelocity * ParticleTarget.TargetVelocityInfluence;
	if (Alpha > KINDA_SMALL_NUMBER && ParentParticlePtr)
	{
		RefVelocity -= Alpha * ParentParticlePtr->TargetVelocity * ParticleTarget.TargetVelocityInfluence;
	}

	// A = alpha~ in the Mueller paper
	const float A = ParticleTarget.Compliance * FMath::Square(InvDeltaTime);
	// G = gamma in the Mueller paper = alpha~ * beta~ / dt
	const float G = ParticleTarget.Compliance * ParticleTarget.Damping * InvDeltaTime;

	const FSimVector RelVelocity = CurrentVelocity - RefVelocity;

	// XPBD update (Jacobian = I for child, -Alpha*I for parent)
	const float Denom = (1.0f + G) * InvMassForDenom + A;
	const FSimVector DeltaLambda =
		(-C - A * ParticleTarget.Lambda - G * RelVelocity * DeltaTime) / Denom;

	ParticleTarget.Lambda += DeltaLambda;
	Particle.Position += DeltaLambda * Particle.InvMass;

	// Parent correction scaled by Alpha
	if (Alpha > KINDA_SMALL_NUMBER && ParentParticlePtr && !bOnlyMoveChild)
	{
		ParentParticlePtr->Position -= Alpha * DeltaLambda * ParentParticlePtr->InvMass;
	}
}

//======================================================================================================================
// Constrains the direction from parent to child to lie within AngleLimit of the target
// (animation) direction. The correction is scaled by a compliance-derived factor so that low
// AngleLimitStrength gives soft/springy behaviour. Uses inverse-mass weighting between child
// and parent. When bOnlyMoveChild is true, only the child is moved.
static void SolveTargetAngleConstraint(
	FParticle&              Particle,
	const FParticleTarget&  ParticleTarget,
	FParticle*              ParentParticlePtr,
	const bool              bOnlyMoveChild,
	const float             InvDeltaTime)
{
	if (ParticleTarget.AngleLimitCompliance <= 0.0f || !ParentParticlePtr)
	{
		return;
	}

	if (ParticleTarget.TargetDirectionFromParent.IsNearlyZero(KINDA_SMALL_NUMBER))
	{
		return;
	}

	const FSimVector Delta = Particle.Position - ParentParticlePtr->Position;
	const FSimScalar DeltaLen = Delta.Size();
	if (DeltaLen < KINDA_SMALL_NUMBER)
	{
		return;
	}

	const FSimVector CurrentDir = Delta * (1.0f / DeltaLen);
	const FSimVector& TargetDir = ParticleTarget.TargetDirectionFromParent;

	const FSimScalar CosTheta = FMath::Clamp(
		FSimVector::DotProduct(CurrentDir, TargetDir),
		FSimScalar(-1.0f + KINDA_SMALL_NUMBER), FSimScalar(1.0f - KINDA_SMALL_NUMBER));

	// Equivalent to Theta <= AngleLimit
	if (CosTheta >= ParticleTarget.CosAngleLimit)
	{
		return;
	}

	// Rotation axis to bring CurrentDir toward TargetDir
	FSimVector Axis = FSimVector::CrossProduct(CurrentDir, TargetDir);
	const FSimScalar AxisLen = Axis.Size();
	if (AxisLen < KINDA_SMALL_NUMBER)
	{
		return;
	}
	Axis *= (1.0f / AxisLen);

	// Rotate Delta toward target direction by the excess angle (Theta - Limit) using
	// Rodrigues' formula. dot(Axis, Delta) is zero so it simplifies to:
	//   NewDelta = Delta * cos(Excess) + cross(Axis, Delta) * sin(Excess)
	//
	// We avoid trig on the excess angle via the subtraction identities:
	//   cos(Theta - Limit) = cosTheta * cosLimit + sinTheta * sinLimit
	//   sin(Theta - Limit) = sinTheta * cosLimit - cosTheta * sinLimit
	const FSimScalar SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);
	const FSimScalar CosExcess = CosTheta * ParticleTarget.CosAngleLimit + SinTheta * ParticleTarget.SinAngleLimit;
	const FSimScalar SinExcess = SinTheta * ParticleTarget.CosAngleLimit - CosTheta * ParticleTarget.SinAngleLimit;
	const FSimVector NewDelta = Delta * CosExcess + FSimVector::CrossProduct(Axis, Delta) * SinExcess;

	// Scale the correction by a compliance-derived factor. When compliance is 0 (very high
	// strength) Alpha is 1 and the correction is fully applied. When compliance is large (low
	// strength) Alpha approaches 0 and the correction is soft.
	const float InvMassSum = Particle.InvMass + (bOnlyMoveChild ? 0.0f : ParentParticlePtr->InvMass);
	// A = alpha~ in the Mueller paper
	const float A = ParticleTarget.AngleLimitCompliance * FMath::Square(InvDeltaTime);
	const float Alpha = InvMassSum / (InvMassSum + A);
	const FSimVector Correction = (NewDelta - Delta) * Alpha;

	if (bOnlyMoveChild)
	{
		Particle.Position += Correction;
	}
	else
	{
		if (InvMassSum > KINDA_SMALL_NUMBER)
		{
			Particle.Position += Correction * (Particle.InvMass / InvMassSum);
			ParentParticlePtr->Position -= Correction * (ParentParticlePtr->InvMass / InvMassSum);
		}
	}
}

//======================================================================================================================
static void SolveHardDistanceConstraint(
	const FHardDistanceConstraint& Constraint,
	FParticle&                     ParentParticle,
	FParticle&                     ChildParticle,
	const bool                     bOnlyMoveChild)
{
	if (bOnlyMoveChild)
	{
		if (ChildParticle.InvMass == 0)
		{
			return;
		}

		const FSimVector Delta = ChildParticle.Position - ParentParticle.Position;
		const FSimScalar DeltaLen = Delta.Size();
		if (DeltaLen < KINDA_SMALL_NUMBER)
		{
			return;
		}

		const FSimVector N = Delta * (1.0f / DeltaLen);
		ChildParticle.Position += N * (Constraint.TargetDistance - DeltaLen);
	}
	else
	{
		const FSimVector Delta = ChildParticle.Position - ParentParticle.Position;
		const FSimScalar DeltaLen = Delta.Size();
		if (DeltaLen < KINDA_SMALL_NUMBER)
		{
			return;
		}

		const float InvMassParent = ParentParticle.InvMass;
		const float InvMassChild = ChildParticle.InvMass;
		const float InvMassSum = InvMassParent + InvMassChild;
		if (InvMassSum == 0)
		{
			return;
		}

		const FSimVector N = Delta * (1.0f / DeltaLen);
		const FSimVector Correction = N * ((Constraint.TargetDistance - DeltaLen) / InvMassSum);

		ParentParticle.Position -= Correction * InvMassParent;
		ChildParticle.Position += Correction * InvMassChild;
	}
}

//======================================================================================================================
static void SolveSoftDistanceConstraint(
	FSoftDistanceConstraint& Constraint,
	FParticle&               ParentParticle,
	FParticle&               ChildParticle,
	const float              DeltaTime,
	const float              InvDeltaTime)
{
	const float InvMassParent = ParentParticle.InvMass;
	const float InvMassChild = ChildParticle.InvMass;
	const float InvMassSum = InvMassParent + InvMassChild;
	if (InvMassSum == 0)
	{
		return;
	}

	const FSimVector Delta = ChildParticle.Position - ParentParticle.Position;
	const FSimScalar DeltaLen = Delta.Size();
	if (DeltaLen < KINDA_SMALL_NUMBER)
	{
		return;
	}

	const FSimVector N = Delta * (1.0f / DeltaLen);
	const FSimScalar C = DeltaLen - Constraint.TargetDistance; // constraint error

	// A = alpha~ in the Mueller paper
	const float A = Constraint.Compliance * FMath::Square(InvDeltaTime);
	// The paper uses B = beta~ = FMath::Square(DeltaTime) * Constraint.Damping;
	// G = gamma in the Mueller paper = alpha~ * beta~ / dt
	const float G = Constraint.Compliance * Constraint.Damping * InvDeltaTime;

	// Compute relative velocity from positions (no stored velocity field)
	const FSimVector RelVelocity =
		(ChildParticle.Position - ChildParticle.PrevPosition
			- ParentParticle.Position + ParentParticle.PrevPosition) * InvDeltaTime;
	const FSimScalar dCdt = N.Dot(RelVelocity);

	// Solve
	const FSimScalar Num = -C - A * Constraint.Lambda - G * dCdt * DeltaTime;
	const float Denom = (1.0f + G) * InvMassSum + A;
	const FSimScalar DeltaLambda = Num / Denom;

	Constraint.Lambda += DeltaLambda;

	// Apply
	FSimVector Correction = N * DeltaLambda;
	ParentParticle.Position -= Correction * InvMassParent;
	ChildParticle.Position += Correction * InvMassChild;
}

//======================================================================================================================
// Enforces the cone limit: the angle between the incoming bone direction (grandparent->parent)
// and outgoing bone direction (parent->child) must not exceed the limit angle. Uses XPBD with
// compliance and damping. Corrections are applied to all three particles with mass weighting,
// and the gradients sum to zero so the centre of mass is preserved.
static void SolveConeLimit(
	FConeLimit& ConeLimit,
	FParticle&  GrandparentParticle,
	FParticle&  ParentParticle,
	FParticle&  ChildParticle,
	const float DeltaTime,
	const float InvDeltaTime)
{
	const float InvMassG = GrandparentParticle.InvMass;
	const float InvMassP = ParentParticle.InvMass;
	const float InvMassK = ChildParticle.InvMass;

	if (InvMassG + InvMassP + InvMassK < KINDA_SMALL_NUMBER)
	{
		return; // All kinematic
	}

	// Bone directions: incoming (grandparent->parent), outgoing (parent->child)
	const FSimVector D1 = ParentParticle.Position - GrandparentParticle.Position;
	const FSimVector D2 = ChildParticle.Position - ParentParticle.Position;
	const FSimScalar R1 = D1.Size();
	const FSimScalar R2 = D2.Size();

	if (R1 < KINDA_SMALL_NUMBER || R2 < KINDA_SMALL_NUMBER)
	{
		return;
	}

	const FSimVector N1 = D1 * (1.0f / R1);
	const FSimVector N2 = D2 * (1.0f / R2);

	const FSimScalar CosTheta = FMath::Clamp(
		FSimVector::DotProduct(N1, N2),
		FSimScalar(-1.0f + KINDA_SMALL_NUMBER), FSimScalar(1.0f - KINDA_SMALL_NUMBER));
	const FSimScalar SinTheta = FMath::Sqrt(1.0f - CosTheta * CosTheta);

	if (SinTheta < KINDA_SMALL_NUMBER)
	{
		return; // Degenerate: nearly straight or fully folded
	}

	const FSimScalar Theta = FMath::Acos(CosTheta);
	const FSimScalar C = Theta - ConeLimit.Angle;

	if (C <= 0.0f)
	{
		return; // Within the limit
	}

	const FSimScalar InvSinTheta = 1.0f / SinTheta;

	// Gradients of theta w.r.t. each particle position. These sum to zero, preserving centre
	// of mass when corrections are applied with inverse-mass weighting.
	const FSimVector GradG = (N2 - CosTheta * N1) * (InvSinTheta / R1);
	const FSimVector GradK = -(N1 - CosTheta * N2) * (InvSinTheta / R2);
	const FSimVector GradP = -(GradG + GradK);

	// Weighted sum of squared gradient norms
	const FSimScalar W = InvMassG * GradG.SizeSquared()
		+ InvMassP * GradP.SizeSquared()
		+ InvMassK * GradK.SizeSquared();

	if (W < KINDA_SMALL_NUMBER)
	{
		return;
	}

	// XPBD compliance and damping terms
	// A = alpha~ in the Mueller paper
	const float A = ConeLimit.Compliance * FMath::Square(InvDeltaTime);
	// G = gamma in the Mueller paper = alpha~ * beta~ / dt
	const float G = ConeLimit.Compliance * ConeLimit.Damping * InvDeltaTime;

	// Constraint velocity for damping
	const FSimVector VelG = (GrandparentParticle.Position - GrandparentParticle.PrevPosition) * InvDeltaTime;
	const FSimVector VelP = (ParentParticle.Position - ParentParticle.PrevPosition) * InvDeltaTime;
	const FSimVector VelK = (ChildParticle.Position - ChildParticle.PrevPosition) * InvDeltaTime;
	const FSimScalar dCdt =
		FSimVector::DotProduct(GradG, VelG) + FSimVector::DotProduct(GradP, VelP) + FSimVector::DotProduct(GradK, VelK);

	// Solve
	const FSimScalar Num = -C - A * ConeLimit.Lambda - G * dCdt * DeltaTime;
	const FSimScalar Denom = (1.0f + G) * W + A;
	const FSimScalar DeltaLambda = Num / Denom;

	ConeLimit.Lambda += DeltaLambda;

	// Apply corrections
	GrandparentParticle.Position += GradG * (InvMassG * DeltaLambda);
	ParentParticle.Position += GradP * (InvMassP * DeltaLambda);
	ChildParticle.Position += GradK * (InvMassK * DeltaLambda);
}

//======================================================================================================================
// Note that the docs in the Collider Components note that scale is ignored
static void SolveContactConstraint(
	const FBoxShape&          BoxShape,
	const FSimTransform&      ColliderTM,
	FParticle&                Particle,
	const FParticleCollider& ParticleCollider)
{
	const FSimTransform TM = BoxShape.TM * ColliderTM;

	const FSimVector HalfExtents = BoxShape.Extents * 0.5f;
	const FSimVector LocalPos = TM.InverseTransformPositionNoScale(Particle.Position);

	// Signed distance from particle center to each box face (positive = inside the box)
	const float DPosX = HalfExtents.X - LocalPos.X;
	const float DNegX = HalfExtents.X + LocalPos.X;
	const float DPosY = HalfExtents.Y - LocalPos.Y;
	const float DNegY = HalfExtents.Y + LocalPos.Y;
	const float DPosZ = HalfExtents.Z - LocalPos.Z;
	const float DNegZ = HalfExtents.Z + LocalPos.Z;

	FSimVector N;
	float C;

	if (DPosX >= 0.0f && DNegX >= 0.0f &&
		DPosY >= 0.0f && DNegY >= 0.0f &&
		DPosZ >= 0.0f && DNegZ >= 0.0f)
	{
		// Particle center is inside (or on surface of) the box - push out through nearest face
		float MinDist = DPosX;
		FSimVector LocalNormal(1.0f, 0.0f, 0.0f);

		if (DNegX < MinDist) {
			MinDist = DNegX; 
			LocalNormal = FSimVector(-1.0f, 0.0f, 0.0f); 
		}
		if (DPosY < MinDist)
		{ 
			MinDist = DPosY; 
			LocalNormal = FSimVector(0.0f, 1.0f, 0.0f); 
		}
		if (DNegY < MinDist) 
		{ 
			MinDist = DNegY; 
			LocalNormal = FSimVector(0.0f, -1.0f, 0.0f);
		}
		if (DPosZ < MinDist) 
		{
			MinDist = DPosZ; 
			LocalNormal = FSimVector(0.0f, 0.0f, 1.0f); 
		}
		if (DNegZ < MinDist) 
		{
			MinDist = DNegZ; 
			LocalNormal = FSimVector(0.0f, 0.0f, -1.0f); 
		}

		N = TM.TransformVectorNoScale(LocalNormal);
		C = MinDist + ParticleCollider.Radius;
	}
	else
	{
		// Particle center is outside the box - find closest point on box surface
		const FSimVector ClampedPos(
			FMath::Clamp(LocalPos.X, -HalfExtents.X, HalfExtents.X),
			FMath::Clamp(LocalPos.Y, -HalfExtents.Y, HalfExtents.Y),
			FMath::Clamp(LocalPos.Z, -HalfExtents.Z, HalfExtents.Z));
		const FSimVector ClosestPoint = TM.TransformPositionNoScale(ClampedPos);
		const FSimVector Delta = Particle.Position - ClosestPoint;
		const FSimScalar DeltaLenSq = Delta.SizeSquared();
		if (DeltaLenSq > FMath::Square(ParticleCollider.Radius) || DeltaLenSq < KINDA_SMALL_NUMBER)
		{
			return;
		}

		const FSimScalar DeltaLen = FMath::Sqrt(DeltaLenSq);
		N = Delta * (1.0f / DeltaLen);
		C = ParticleCollider.Radius - DeltaLen;
	}

	Particle.Position += N * C;
}

//======================================================================================================================
// Note that the docs in the Collider Components note that scale is ignored
static void SolveContactConstraint(
	const FCapsuleShape&      CapsuleShape,
	const FSimTransform&      ColliderTM,
	FParticle&                Particle,
	const FParticleCollider&  ParticleCollider)
{
	const FSimTransform TM = CapsuleShape.TM * ColliderTM;

	const float MinDistance = CapsuleShape.Radius + ParticleCollider.Radius;
	if (MinDistance <= 0.0f)
	{
		return;
	}

	FSimVector SegmentPoint;
	if (CapsuleShape.Length < KINDA_SMALL_NUMBER)
	{
		SegmentPoint = TM.GetLocation();
	}
	else
	{
		// Inline closest-point-on-segment to avoid potential FVector3f overload issues
		const FSimVector SegA = TM.TransformPositionNoScale(FSimVector(0, 0, -CapsuleShape.Length * 0.5f));
		const FSimVector SegB = TM.TransformPositionNoScale(FSimVector(0, 0, CapsuleShape.Length * 0.5f));
		const FSimVector Seg = SegB - SegA;
		const float T = FMath::Clamp(
			FSimVector::DotProduct(Particle.Position - SegA, Seg) /
			FMath::Max(FSimVector::DotProduct(Seg, Seg), KINDA_SMALL_NUMBER),
			0.0f, 1.0f);
		SegmentPoint = SegA + T * Seg;
	}

	const FSimVector Delta = Particle.Position - SegmentPoint;
	const FSimScalar DeltaLenSq = Delta.SizeSquared();
	if (DeltaLenSq > FMath::Square(MinDistance) || DeltaLenSq < KINDA_SMALL_NUMBER)
	{
		return;
	}

	const FSimScalar DeltaLen = FMath::Sqrt(DeltaLenSq);
	const FSimVector N = Delta * (1.0f / DeltaLen);
	const FSimScalar C = MinDistance - DeltaLen;

	Particle.Position += N * C;
}

//======================================================================================================================
// Note that the docs in the Collider Components note that scale is ignored
static void SolveContactConstraint(
	const FPlaneShape&        PlaneShape,
	const FSimTransform&      ColliderTM,
	FParticle&                Particle,
	const FParticleCollider&  ParticleCollider)
{
	const FSimTransform TM = PlaneShape.TM * ColliderTM;

	const FSimVector LocalPos = TM.InverseTransformPositionNoScale(Particle.Position);

	// Plane normal is +Z in local space. Signed distance is positive on the normal side.
	const float SignedDist = LocalPos.Z;
	if (SignedDist >= ParticleCollider.Radius)
	{
		return;
	}

	// Reject if particle center is outside
	const FSimVector2D HalfExtent = PlaneShape.Extents * 0.5f;
	if (FMath::Abs(LocalPos.X) > HalfExtent.X || FMath::Abs(LocalPos.Y) > HalfExtent.Y)
	{
		return;
	}

	// Push-out is in the plane normal direction
	const FSimVector N = TM.TransformVectorNoScale(FSimVector(0, 0, 1));
	const float C = ParticleCollider.Radius - SignedDist;

	Particle.Position += N * C;
}

//======================================================================================================================
static void SolveColliderContacts(
	FParticle&                         Particle,
	const FParticleCollider&           ParticleCollider,
	const FParticleNoCollision&        ParticleNoCollision,
	TArrayView<const FShapeCollection> Colliders)
{
	if (Particle.InvMass == 0.0f || !ParticleCollider.bCollideWithColliders)
	{
		return;
	}
	// Note that all the contact constraints here are hard and have no friction. This makes them
	// very simple and fast, not requiring any per-contact state (i.e. no lambda for the normal and
	// tangential force). We should anticipate adding a more fully-featured contact constraint that
	// supports compliance and friction, enabled on a per-collider basis.
	const int32 NumColliders = Colliders.Num();
	for (int32 ColliderIndex = 0; ColliderIndex < NumColliders; ++ColliderIndex)
	{
		if (ParticleNoCollision.NoCollisionColliderIndices[ColliderIndex])
		{
			continue;
		}

		const FShapeCollection& Collider = Colliders[ColliderIndex];

		for (const FBoxShape& BoxShape : Collider.BoxShapes)
		{
			SolveContactConstraint(BoxShape, Collider.TM, Particle, ParticleCollider);
		}
		for (const FCapsuleShape& CapsuleShape : Collider.CapsuleShapes)
		{
			SolveContactConstraint(CapsuleShape, Collider.TM, Particle, ParticleCollider);
		}
		for (const FPlaneShape& PlaneShape : Collider.PlaneShapes)
		{
			SolveContactConstraint(PlaneShape, Collider.TM, Particle, ParticleCollider);
		}
	}
}

//======================================================================================================================
// Computes how far a particle's centre is outside a box-shaped confiner, along with the inward
// normal needed to push it back inside. Returns false when the centre is inside. Confiners only
// constrain the particle position; its radius is ignored.
static bool ComputeConfinerPenetration(
	const FBoxShape&          BoxShape,
	const FSimTransform&      ConfinerTM,
	const FParticle&          Particle,
	FSimVector&               OutInwardNormal,
	FSimScalar&               OutPenetration)
{
	const FSimTransform TM = BoxShape.TM * ConfinerTM;
	const FSimVector HalfExtents = BoxShape.Extents * 0.5f;
	const FSimVector LocalPos = TM.InverseTransformPositionNoScale(Particle.Position);

	// If every face distance is non-negative the centre is inside the box - nothing to do.
	if (FMath::Abs(LocalPos.X) <= HalfExtents.X &&
		FMath::Abs(LocalPos.Y) <= HalfExtents.Y &&
		FMath::Abs(LocalPos.Z) <= HalfExtents.Z)
	{
		return false;
	}

	// Particle centre is outside - find the nearest point on the box surface and push toward it.
	const FSimVector ClampedPos(
		FMath::Clamp(LocalPos.X, -HalfExtents.X, HalfExtents.X),
		FMath::Clamp(LocalPos.Y, -HalfExtents.Y, HalfExtents.Y),
		FMath::Clamp(LocalPos.Z, -HalfExtents.Z, HalfExtents.Z));
	const FSimVector ClosestPoint = TM.TransformPositionNoScale(ClampedPos);
	const FSimVector Delta = ClosestPoint - Particle.Position;
	const FSimScalar DeltaLenSq = Delta.SizeSquared();
	if (DeltaLenSq < KINDA_SMALL_NUMBER)
	{
		return false;
	}
	const FSimScalar DeltaLen = FMath::Sqrt(DeltaLenSq);
	OutInwardNormal = Delta * (1.0f / DeltaLen);
	OutPenetration = DeltaLen;
	return true;
}

//======================================================================================================================
// Computes how far a particle's centre is outside a capsule-shaped confiner. Only the particle
// position is constrained; its radius is ignored.
static bool ComputeConfinerPenetration(
	const FCapsuleShape&      CapsuleShape,
	const FSimTransform&      ConfinerTM,
	const FParticle&          Particle,
	FSimVector&               OutInwardNormal,
	FSimScalar&               OutPenetration)
{
	const FSimTransform TM = CapsuleShape.TM * ConfinerTM;

	FSimVector SegmentPoint;
	if (CapsuleShape.Length < KINDA_SMALL_NUMBER)
	{
		SegmentPoint = TM.GetLocation();
	}
	else
	{
		const FSimVector SegA = TM.TransformPositionNoScale(FSimVector(0, 0, -CapsuleShape.Length * 0.5f));
		const FSimVector SegB = TM.TransformPositionNoScale(FSimVector(0, 0,  CapsuleShape.Length * 0.5f));
		const FSimVector Seg = SegB - SegA;
		const float T = FMath::Clamp(
			FSimVector::DotProduct(Particle.Position - SegA, Seg) /
			FMath::Max(FSimVector::DotProduct(Seg, Seg), KINDA_SMALL_NUMBER),
			0.0f, 1.0f);
		SegmentPoint = SegA + T * Seg;
	}

	const FSimVector Delta = SegmentPoint - Particle.Position;
	const FSimScalar DeltaLenSq = Delta.SizeSquared();
	const FSimScalar MaxSq = FSimScalar(CapsuleShape.Radius) * FSimScalar(CapsuleShape.Radius);
	if (DeltaLenSq <= MaxSq)
	{
		return false;
	}
	const FSimScalar DeltaLen = FMath::Sqrt(DeltaLenSq);
	OutInwardNormal = Delta * (1.0f / DeltaLen);
	OutPenetration = DeltaLen - FSimScalar(CapsuleShape.Radius);
	return true;
}

//======================================================================================================================
// Computes how far a particle's centre is on the wrong side of a plane-shaped confiner. The plane
// is treated as an infinite +Z half-space; the stored Extents are used only for visualization,
// not for confinement. Only the particle position is constrained; its radius is ignored.
static bool ComputeConfinerPenetration(
	const FPlaneShape&        PlaneShape,
	const FSimTransform&      ConfinerTM,
	const FParticle&          Particle,
	FSimVector&               OutInwardNormal,
	FSimScalar&               OutPenetration)
{
	const FSimTransform TM = PlaneShape.TM * ConfinerTM;
	const FSimVector LocalPos = TM.InverseTransformPositionNoScale(Particle.Position);

	// Local +Z is the "inside" side; signed distance is LocalPos.Z.
	const float SignedDist = LocalPos.Z;
	if (SignedDist >= 0.0f)
	{
		return false;
	}

	OutInwardNormal = TM.TransformVectorNoScale(FSimVector(0, 0, 1));
	OutPenetration = FSimScalar(-SignedDist);
	return true;
}

//======================================================================================================================
// Keep a particle's position inside every shape of every confiner it has opted in to. Each shape
// is enforced independently (intersection semantics). Uses the damping-free compliance-scaled
// correction form (see SolveTargetAngleConstraint) - no lambda accumulation. Only the particle
// position is constrained; its radius is ignored.
static void SolveConfinerContacts(
	FParticle&                         Particle,
	const FParticleConfinement&        ParticleConfinement,
	TArrayView<const FShapeCollection> Confiners,
	TArrayView<const float>            ConfinerStrengths,
	const float                        InvDeltaTime)
{
	if (Particle.InvMass == 0.0f)
	{
		return;
	}

	for (const int32 ConfinerIndex : ParticleConfinement.ConfinerIndices)
	{
		if (!Confiners.IsValidIndex(ConfinerIndex) || !ConfinerStrengths.IsValidIndex(ConfinerIndex))
		{
			continue;
		}
		const FShapeCollection& Confiner = Confiners[ConfinerIndex];

		// Alpha = 1 / (1 + (InvDt / W)^2), where W = Strength * 2pi. Independent of InvMass because
		// Compliance = InvMass / W^2 in the XPBD formulation and the InvMass cancels in the Alpha
		// ratio. Strength <= 0 disables the confiner.
		const float Strength = ConfinerStrengths[ConfinerIndex];
		if (Strength <= 0.0f)
		{
			continue;
		}
		const float W = Strength * TWO_PI;
		const float Ratio = InvDeltaTime / W;
		const float Alpha = 1.0f / (1.0f + Ratio * Ratio);

		FSimVector N;
		FSimScalar C;

		for (const FBoxShape& Box : Confiner.BoxShapes)
		{
			if (ComputeConfinerPenetration(Box, Confiner.TM, Particle, N, C))
			{
				Particle.Position += N * (C * Alpha);
			}
		}
		for (const FCapsuleShape& Capsule : Confiner.CapsuleShapes)
		{
			if (ComputeConfinerPenetration(Capsule, Confiner.TM, Particle, N, C))
			{
				Particle.Position += N * (C * Alpha);
			}
		}
		for (const FPlaneShape& Plane : Confiner.PlaneShapes)
		{
			if (ComputeConfinerPenetration(Plane, Confiner.TM, Particle, N, C))
			{
				Particle.Position += N * (C * Alpha);
			}
		}
	}
}

//======================================================================================================================
static void SolveParticleToParticleContact(
	FParticle& ParticleA, const FParticleCollider& ParticleACollider,
	FParticle& ParticleB, const FParticleCollider& ParticleBCollider)
{
	const FSimScalar CombinedRadius = ParticleACollider.Radius + ParticleBCollider.Radius;

	const FSimVector Delta = ParticleA.Position - ParticleB.Position;
	const FSimScalar DeltaLenSq = Delta.SizeSquared();
	if (DeltaLenSq > FMath::Square(CombinedRadius) || DeltaLenSq < KINDA_SMALL_NUMBER)
	{
		return;
	}

	const float InvMassA = ParticleA.InvMass;
	const float InvMassB = ParticleB.InvMass;
	const float InvMassSum = InvMassA + InvMassB;
	if (InvMassSum == 0)
	{
		return;
	}

	const FSimScalar DeltaLen = FMath::Sqrt(DeltaLenSq);
	const FSimVector N = Delta * (1.0f / DeltaLen);
	const FSimVector Correction = N * ((CombinedRadius - DeltaLen) / InvMassSum);

	ParticleA.Position += Correction * InvMassA;
	ParticleB.Position -= Correction * InvMassB;
}

//======================================================================================================================
// The functions above here should be generic, just dealing with individual components of a simulation.
// 
// The functions below here are directly related to FSimulationState which is a specific simulation
// configuration, oriented towards animation/skeletal dynamics.
//======================================================================================================================

//======================================================================================================================
void UpdateParticleTarget(
	FSimulationState&  State,
	const int32        Index,
	float              InvDeltaTime,
	const FSimVector&  InTargetPosition,
	const float        InStrength,
	const float        InDampingRatio,
	const float        InExtraDamping,
	const float        InTargetVelocityInfluence,
	const float        InTargetMode,
	const float        InAngleLimit,
	const float        InAngleLimitStrength,
	const bool         bInAccelerationMode)
{
	FParticleInfo& Info = State.ParticleInfos[Index];
	const FParticle& Particle = State.Particles[Index];
	FParticleTarget& ParticleTarget = State.ParticleTargets[Index];

	// Shift previous targets
	Info.PreviousTargetPosition = Info.TargetPosition;
	Info.PreviousTargetVelocity = Info.TargetVelocity;

	// Set new target and compute velocity
	Info.TargetPosition = InTargetPosition;
	Info.TargetVelocity = (Info.TargetPosition - Info.PreviousTargetPosition) * InvDeltaTime;

	if (Particle.InvMass == 0.0f)
	{
		// FParticleTarget and the other Info properties are never read for kinematic particles.
		return;
	}

	// Clamp Strengths to >= 0. ClampMin on the UPROPERTYs only catches editor entry; Set nodes and
	// programmatic writes can still push negative values. Without this, W = Strength * TWO_PI would
	// flip negative and the 2*ratio*W damping term would become negative (energy injection) while
	// Compliance = InvMass / W^2 would still produce the same stiffness as |Strength|.
	const float Strength = FMath::Max(InStrength, 0.0f);
	const float AngleLimitStrength = FMath::Max(InAngleLimitStrength, 0.0f);

	// Unified parent lookup. ParticleInfos and Particles are parallel arrays with identical size,
	// so a single validity check covers both.
	const bool bHasParent = State.Particles.IsValidIndex(Info.ParentParticleIndex);
	const FParticle* ParentParticle = bHasParent ? &State.Particles[Info.ParentParticleIndex] : nullptr;
	const FParticleInfo* ParentInfo = bHasParent ? &State.ParticleInfos[Info.ParentParticleIndex] : nullptr;

	// Compute and store the normalized direction from parent to this particle in the target pose.
	// Needed when TargetMode > 0 (directional contribution) or when angle limits are active.
	// It is intentional that both directions use the current ParentParticleTargetPosition.
	const bool bAngleLimitActive = AngleLimitStrength > KINDA_SMALL_NUMBER;
	if ((InTargetMode > KINDA_SMALL_NUMBER || bAngleLimitActive) && ParentInfo)
	{
		const FSimVector& ParentTargetPos = ParentInfo->TargetPosition;
		Info.PreviousTargetDirectionFromParent =
			GetSafeNormalFast(Info.PreviousTargetPosition - ParentTargetPos);
		Info.TargetDirectionFromParent = GetSafeNormalFast(Info.TargetPosition - ParentTargetPos);
	}

	// Angle limit (compliance from strength, same pattern as the other constraints)
	if (bAngleLimitActive && ParentParticle)
	{
		const float AngleLimitRadians = FMath::DegreesToRadians(FMath::Max(InAngleLimit, 0.0f));
		FMath::SinCos(&ParticleTarget.SinAngleLimit, &ParticleTarget.CosAngleLimit, AngleLimitRadians);

		const float W = AngleLimitStrength * TWO_PI;
		const float InvMassSum = Particle.InvMass + ParentParticle->InvMass;
		// Angle limit is acceleration-mode only - bInAccelerationMode does not gate this. Force-mode
		// behaviour on the angle limit is not supported (the limit would become useless on heavy
		// particles, which would deflect under the parent's own corrective force).
		ParticleTarget.AngleLimitCompliance = InvMassSum / FMath::Max(W * W, KINDA_SMALL_NUMBER);
	}
	else
	{
		ParticleTarget.CosAngleLimit = 1.0f;
		ParticleTarget.SinAngleLimit = 0.0f;
		ParticleTarget.AngleLimitCompliance = 0.0f;
	}

	ParticleTarget.TargetVelocityInfluence = InTargetVelocityInfluence;
	ParticleTarget.TargetMode = InTargetMode;

	// In acceleration mode (default) the effective inverse mass is folded into compliance/damping
	// so the constraint's natural frequency is mass-independent. Parent contribution is weighted by
	// Alpha^2 (from the squared Jacobian). In force mode the mass term drops out, so heavier
	// particles oscillate more slowly (period proportional to sqrt(mass)).
	const float Alpha = ParentParticle ? InTargetMode : 0.0f;
	const float ParentInvMass = ParentParticle ? ParentParticle->InvMass : 0.0f;
	const float InvMass = Particle.InvMass + Alpha * Alpha * ParentInvMass;
	const float W = Strength * TWO_PI;
	const float WSquaredSafe = FMath::Max(W * W, KINDA_SMALL_NUMBER);
	if (bInAccelerationMode)
	{
		ParticleTarget.Compliance = InvMass / WSquaredSafe;
		ParticleTarget.Damping =
			(2.0f * InDampingRatio * W + InExtraDamping) / FMath::Max(InvMass, KINDA_SMALL_NUMBER);
	}
	else
	{
		ParticleTarget.Compliance = 1.0f / WSquaredSafe;
		ParticleTarget.Damping = 2.0f * InDampingRatio * W + InExtraDamping;
	}
}

//======================================================================================================================
void UpdateDynamicsStep(
	FSimulationState&              SimulationState,
	const FSolverSettings&         SolverSettings,
	const FSimulationSpaceMotion&  SimulationSpaceMotion,
	const float                    DeltaTime,
	const int32                    StepIndex,
	const int32                    NumSteps)
{
	if (DeltaTime < SMALL_NUMBER)
	{
		return;
	}
	const float StepAlpha = (float)(StepIndex + 1) / (float)NumSteps;
	const float InvDeltaTime = 1.0f / DeltaTime;

	// Time-corrected Verlet: scale the implicit velocity by dt_current / dt_prev
	const float DtRatio =
		(SimulationState.PrevStepTime > SMALL_NUMBER) ? (DeltaTime / SimulationState.PrevStepTime) : 1.0f;

	// These array sizes are invariant across this step; hoist so the compiler sees that.
	const int32 NumParticles           = SimulationState.Particles.Num();
	const int32 NumSoftConstraints     = SimulationState.SoftDistanceConstraints.Num();
	const int32 NumConeLimits          = SimulationState.ConeLimits.Num();
	const int32 NumHardConstraints     = SimulationState.HardDistanceConstraints.Num();
	const int32 NumSkeletalConstraints = SimulationState.SkeletalConstraints.Num();

	// Step all the things - this means to update them for the (sub) step, prior to solving constraints
	for (FSoftDistanceConstraint& Constraint : SimulationState.SoftDistanceConstraints)
	{
		StepSoftDistanceConstraint(Constraint);
	}

	for (FConeLimit& ConeLimit : SimulationState.ConeLimits)
	{
		StepConeLimit(ConeLimit);
	}

	// Hard distance constraints don't have a step

	for (FShapeCollection& Collider : SimulationState.Colliders)
	{
		StepShapeCollection(Collider, StepAlpha);
	}

	for (FShapeCollection& Confiner : SimulationState.Confiners)
	{
		StepShapeCollection(Confiner, StepAlpha);
	}

	for (int32 Index = 0; Index < NumParticles; ++Index)
	{
		const FParticleInfo& Info = SimulationState.ParticleInfos[Index];
		FParticle& Particle = SimulationState.Particles[Index];
		FParticleTarget& ParticleTarget = SimulationState.ParticleTargets[Index];

		if (Particle.InvMass != 0.0f)
		{
			StepParticleDynamic(Particle, Info, DeltaTime, DtRatio, StepAlpha,
				FSimVector(Info.GravityMultiplier * SolverSettings.Gravity), SimulationSpaceMotion);
			StepParticleTarget(ParticleTarget, StepAlpha, Info.PreviousTargetDirectionFromParent,
				Info.TargetDirectionFromParent);
		}
		else
		{
			StepParticleKinematic(Particle, Info, DeltaTime, StepAlpha);
		}
	}

	// Iteratively solve constraints

	for (int32 Iter = 0; Iter != SolverSettings.NumIterations; ++Iter)
	{
		const bool bTargetOnlyMoveChild = (Iter == SolverSettings.NumIterations - 1);
		for (int32 Index = 0; Index < NumParticles; ++Index)
		{
			FParticle& Particle = SimulationState.Particles[Index];
			if (Particle.InvMass != 0.0f)
			{
				FParticleTarget& ParticleTarget = SimulationState.ParticleTargets[Index];
				const FParticleInfo& Info = SimulationState.ParticleInfos[Index];
				FParticle* ParentParticlePtr =
					SimulationState.Particles.IsValidIndex(Info.ParentParticleIndex)
					? &SimulationState.Particles[Info.ParentParticleIndex] : nullptr;
				SolveTargetConstraint(
					Particle, ParticleTarget, ParentParticlePtr, bTargetOnlyMoveChild,
					DeltaTime, InvDeltaTime);
				if (ParticleTarget.AngleLimitCompliance > 0.0f)
				{
					SolveTargetAngleConstraint(
						Particle, ParticleTarget, ParentParticlePtr, bTargetOnlyMoveChild,
						InvDeltaTime);
				}
			}
		}

		// Distance constraints etc
		for (int32 ConstraintIter = 0; ConstraintIter != SolverSettings.NumConstraintSubIterations; ++ConstraintIter)
		{
			// Soft constraints can be soft
			for (int32 Index = 0; Index != NumSoftConstraints; ++Index)
			{
				const FSoftDistanceConstraintInfo& Info =
					SimulationState.SoftDistanceConstraintInfos[Index];
				FSoftDistanceConstraint& Constraint =
					SimulationState.SoftDistanceConstraints[Index];

				check(SimulationState.Particles.IsValidIndex(Info.ParentIndex) &&
					SimulationState.Particles.IsValidIndex(Info.ChildIndex));
				SolveSoftDistanceConstraint(
					Constraint, SimulationState.Particles[Info.ParentIndex], SimulationState.Particles[Info.ChildIndex],
					DeltaTime, InvDeltaTime);
			}

			// Cone limits (angular constraints between triples of particles)
			for (int32 Index = 0; Index != NumConeLimits; ++Index)
			{
				const FConeLimitInfo& Info = SimulationState.ConeLimitInfos[Index];
				FConeLimit& ConeLimit = SimulationState.ConeLimits[Index];

				check(SimulationState.Particles.IsValidIndex(Info.GrandparentIndex) &&
					SimulationState.Particles.IsValidIndex(Info.ParentIndex) &&
					SimulationState.Particles.IsValidIndex(Info.ChildIndex));
				SolveConeLimit(
					ConeLimit, SimulationState.Particles[Info.GrandparentIndex],
					SimulationState.Particles[Info.ParentIndex], SimulationState.Particles[Info.ChildIndex],
					DeltaTime, InvDeltaTime);
			}

			// Hard distance constraints come later to reduce stretching
			for (int32 Index = 0; Index != NumHardConstraints; ++Index)
			{
				const FHardDistanceConstraintInfo& Info = SimulationState.HardDistanceConstraintInfos[Index];
				const FHardDistanceConstraint& Constraint = SimulationState.HardDistanceConstraints[Index];

				check(SimulationState.Particles.IsValidIndex(Info.ParentIndex) &&
					SimulationState.Particles.IsValidIndex(Info.ChildIndex));
				SolveHardDistanceConstraint(
					Constraint, SimulationState.Particles[Info.ParentIndex],
					SimulationState.Particles[Info.ChildIndex], false);
			}

			// Collisions between particles
			for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
			{
				FParticle& Particle = SimulationState.Particles[ParticleIndex];
				const FParticleCollider& ParticleCollider = SimulationState.ParticleColliders[ParticleIndex];
				const FParticleToParticleCollision& ParticleToParticleCollision =
					SimulationState.ParticleToParticleCollision[ParticleIndex];
				for (int32 OtherParticleIndex : ParticleToParticleCollision.CollisionParticleIndices)
				{
					check(SimulationState.Particles.IsValidIndex(OtherParticleIndex));
					FParticle& OtherParticle = SimulationState.Particles[OtherParticleIndex];
					const FParticleCollider& OtherParticleCollider = SimulationState.ParticleColliders[OtherParticleIndex];
					SolveParticleToParticleContact(
						Particle, ParticleCollider, OtherParticle, OtherParticleCollider);
				}
			}

			// Collisions should be as hard as possible
			for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
			{
				FParticle& Particle = SimulationState.Particles[ParticleIndex];
				const FParticleCollider& ParticleCollider = SimulationState.ParticleColliders[ParticleIndex];
				const FParticleNoCollision& ParticleNoCollision = SimulationState.ParticleNoCollision[ParticleIndex];
				SolveColliderContacts(Particle, ParticleCollider, ParticleNoCollision, SimulationState.Colliders);
			}

			// Confiners keep opted-in particles inside each of their shapes (soft)
			for (int32 ParticleIndex = 0; ParticleIndex != NumParticles; ++ParticleIndex)
			{
				FParticle& Particle = SimulationState.Particles[ParticleIndex];
				const FParticleConfinement& Confinement = SimulationState.ParticleConfinement[ParticleIndex];
				SolveConfinerContacts(
					Particle, Confinement, SimulationState.Confiners,
					SimulationState.ConfinerStrengths, InvDeltaTime);
			}

			// Skeletal distance constraints come last to eliminate stretching. We also make them
			// one-way on the last iteration.
			bool bOnlyMoveChild =
				(Iter == SolverSettings.NumIterations - 1) &&
				(ConstraintIter == SolverSettings.NumConstraintSubIterations - 1);
			for (int32 Index = 0; Index != NumSkeletalConstraints; ++Index)
			{
				const FHardDistanceConstraintInfo& Info = SimulationState.SkeletalConstraintInfos[Index];
				const FHardDistanceConstraint& Constraint = SimulationState.SkeletalConstraints[Index];

				check(SimulationState.Particles.IsValidIndex(Info.ParentIndex) &&
					SimulationState.Particles.IsValidIndex(Info.ChildIndex));
				SolveHardDistanceConstraint(
					Constraint, SimulationState.Particles[Info.ParentIndex],
					SimulationState.Particles[Info.ChildIndex], bOnlyMoveChild);
			}
		}
	}
}

//======================================================================================================================
void Simulate(
	FSimulationState&              SimulationState,
	const FSolverSettings&         SolverSettings,
	const FSimulationSpaceMotion&  SimulationSpaceMotion,
	const float                    DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_RigParticleSimulation_Simulate);

	if (SolverSettings.MaxNumSteps < 1 || SolverSettings.MaxTimeStep <= SMALL_NUMBER)
	{
		// Solver is effectively disabled (e.g. for debugging). StepShapeCollection is what would
		// otherwise advance Collider.TM / Confiner.TM to TargetTM via substep interpolation, so
		// snap them here so debug visualization reads the latest hierarchy-driven positions rather
		// than stale blends from the previous successful step. Real pass-through would never reach
		// this stage, so we're not introducing real cost by doing this.
		for (FShapeCollection& Collider : SimulationState.Colliders)
		{
			Collider.TM = Collider.TargetTM;
		}
		for (FShapeCollection& Confiner : SimulationState.Confiners)
		{
			Confiner.TM = Confiner.TargetTM;
		}
		return;
	}

	const float MaxStep = FMath::Max(SolverSettings.MaxTimeStep, SMALL_NUMBER);
	const int32 NumSteps = FMath::Clamp(FMath::CeilToInt(DeltaTime / MaxStep), 1, SolverSettings.MaxNumSteps);

	// When MaxNumSteps clamped NumSteps down, DeltaTime / NumSteps may exceed MaxStep. Capping
	// here means the simulation runs slow (sim time advances by NumSteps * MaxStep < DeltaTime)
	// rather than taking a too-large step that would explode.
	const float StepTime = FMath::Min(DeltaTime / NumSteps, MaxStep);
	for (int32 StepIndex = 0; StepIndex != NumSteps; ++StepIndex)
	{
		UpdateDynamicsStep(SimulationState, SolverSettings, SimulationSpaceMotion, StepTime, StepIndex, NumSteps);
		// Must be set inside the loop: UpdateDynamicsStep reads PrevStepTime to scale the implicit
		// Verlet velocity. Substeps 1..N-1 of this call need it to equal the preceding substep's
		// StepTime (so DtRatio = 1 and no correction is applied). Hoisting out would leave them
		// reading the previous Simulate call's PrevStepTime, mis-scaling the Verlet velocity.
		SimulationState.PrevStepTime = StepTime;
	}
}

//======================================================================================================================
TArray<int32> SortParticlesRootToLeaf(FSimulationState& State)
{
	const int32 NumParticles = State.Particles.Num();
	if (NumParticles <= 1)
	{
		TArray<int32> Identity;
		if (NumParticles == 1)
		{
			Identity.Add(0);
		}
		return Identity;
	}

	// Build children-of adjacency lists from ParentParticleIndex
	TArray<TArray<int32>> Children;
	Children.SetNum(NumParticles);

	TArray<int32> Queue;
	Queue.Reserve(NumParticles);

	for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ++ParticleIndex)
	{
		const int32 ParentIndex = State.ParticleInfos[ParticleIndex].ParentParticleIndex;
		if (ParentIndex == INDEX_NONE)
		{
			Queue.Add(ParticleIndex); // Root - seed the BFS queue
		}
		else
		{
			check(State.ParticleInfos.IsValidIndex(ParentIndex));
			Children[ParentIndex].Add(ParticleIndex);
		}
	}

	// Breadth First Search (BFS) from all roots - visitation order gives root-to-leaf sorted order
	TArray<int32> SortedOrder;
	SortedOrder.Reserve(NumParticles);

	int32 Head = 0;
	while (Head < Queue.Num())
	{
		const int32 Current = Queue[Head++];
		SortedOrder.Add(Current);
		for (const int32 Child : Children[Current])
		{
			Queue.Add(Child);
		}
	}
	// Cycles are impossible: ParentParticleIndex is derived from the skeleton hierarchy (a tree),
	// so the particle parent graph is always a forest and all particles are reachable from roots.
	check(SortedOrder.Num() == NumParticles);

	// Compute inverse mapping: OldToNew[old_index] = new_index
	TArray<int32> OldToNew;
	OldToNew.SetNum(NumParticles);
	for (int32 NewIndex = 0; NewIndex < NumParticles; ++NewIndex)
	{
		OldToNew[SortedOrder[NewIndex]] = NewIndex;
	}

	// Reorder all particle-parallel arrays using the gather pattern
	{
		TArray<FParticleInfo> TempInfos;                         TempInfos.SetNum(NumParticles);
		TArray<FParticle> TempParticles;                         TempParticles.SetNum(NumParticles);
		TArray<FParticleTarget> TempTargets;                     TempTargets.SetNum(NumParticles);
		TArray<FParticleCollider> TempColliders;                 TempColliders.SetNum(NumParticles);
		TArray<FParticleNoCollision> TempNoCollision;            TempNoCollision.SetNum(NumParticles);
		TArray<FParticleToParticleCollision> TempP2PCollision;   TempP2PCollision.SetNum(NumParticles);
		TArray<FParticleConfinement> TempConfinement;            TempConfinement.SetNum(NumParticles);

		for (int32 NewIndex = 0; NewIndex < NumParticles; ++NewIndex)
		{
			const int32 OldIndex = SortedOrder[NewIndex];
			TempInfos[NewIndex]        = MoveTemp(State.ParticleInfos[OldIndex]);
			TempParticles[NewIndex]    = MoveTemp(State.Particles[OldIndex]);
			TempTargets[NewIndex]      = MoveTemp(State.ParticleTargets[OldIndex]);
			TempColliders[NewIndex]    = MoveTemp(State.ParticleColliders[OldIndex]);
			TempNoCollision[NewIndex]  = MoveTemp(State.ParticleNoCollision[OldIndex]);
			TempP2PCollision[NewIndex] = MoveTemp(State.ParticleToParticleCollision[OldIndex]);
			TempConfinement[NewIndex]  = MoveTemp(State.ParticleConfinement[OldIndex]);
		}

		State.ParticleInfos               = MoveTemp(TempInfos);
		State.Particles                   = MoveTemp(TempParticles);
		State.ParticleTargets             = MoveTemp(TempTargets);
		State.ParticleColliders           = MoveTemp(TempColliders);
		State.ParticleNoCollision         = MoveTemp(TempNoCollision);
		State.ParticleToParticleCollision = MoveTemp(TempP2PCollision);
		State.ParticleConfinement         = MoveTemp(TempConfinement);
	}

	// Remap all stored particle indices using OldToNew. Note that at the current call site
	// (Instantiate, before constraints are created), the constraint and collision-pair arrays
	// are empty. We remap them anyway so the function is correct regardless of when it's called.
	for (int32 I = 0; I < NumParticles; ++I)
	{
		int32& ParentIndex = State.ParticleInfos[I].ParentParticleIndex;
		if (ParentIndex != INDEX_NONE)
		{
			ParentIndex = OldToNew[ParentIndex];
		}
		for (int32& OtherIndex : State.ParticleToParticleCollision[I].CollisionParticleIndices)
		{
			OtherIndex = OldToNew[OtherIndex];
		}
	}
	for (FHardDistanceConstraintInfo& Info : State.SkeletalConstraintInfos)
	{
		Info.ParentIndex = OldToNew[Info.ParentIndex];
		Info.ChildIndex = OldToNew[Info.ChildIndex];
	}
	for (FHardDistanceConstraintInfo& Info : State.HardDistanceConstraintInfos)
	{
		Info.ParentIndex = OldToNew[Info.ParentIndex];
		Info.ChildIndex = OldToNew[Info.ChildIndex];
	}
	for (FSoftDistanceConstraintInfo& Info : State.SoftDistanceConstraintInfos)
	{
		Info.ParentIndex = OldToNew[Info.ParentIndex];
		Info.ChildIndex = OldToNew[Info.ChildIndex];
	}
	for (FConeLimitInfo& Info : State.ConeLimitInfos)
	{
		Info.GrandparentIndex = OldToNew[Info.GrandparentIndex];
		Info.ParentIndex = OldToNew[Info.ParentIndex];
		Info.ChildIndex = OldToNew[Info.ChildIndex];
	}

	return SortedOrder;
}

}
