// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#if UE_BUILD_SHIPPING || UE_BUILD_TEST
DECLARE_LOG_CATEGORY_EXTERN(LogControlRigDynamics, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogControlRigDynamics, Log, All);
#endif

//======================================================================================================================
class FControlRigDynamicsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

