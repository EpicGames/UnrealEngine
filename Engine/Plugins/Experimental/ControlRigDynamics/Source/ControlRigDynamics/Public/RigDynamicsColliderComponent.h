// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyComponents.h"

#include "RigDynamicsData.h"

#include "RigDynamicsColliderComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// A component that can be added to a joint/element that defines how a collision shape can be
// attached to it. DynamicsParticles will collide against these shapes.
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsColliderComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsColliderComponent)

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes, meta = (ShowOnlyInnerProperties))
	FRigDynamicsShapeCollection Shapes;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsCollider"); }
};

#undef UE_API
