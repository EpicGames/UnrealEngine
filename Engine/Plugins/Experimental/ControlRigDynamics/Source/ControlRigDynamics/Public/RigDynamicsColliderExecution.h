// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsColliderComponent.h"
#include "RigDynamicsSolverComponent.h"

#include "RigDynamicsColliderExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics collider as a component on the owner element. These colliders track the
// input pose, but do not track the simulated pose - so don't expect good results if you enable
// collisions between particles and colliders that are attached to bones that are themselves simulated.
// Note: This node only runs as part of the construction event.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Collider", Keywords = "Add,Construction,Create,New,Shape,Collision", Varying))
struct FRigUnit_SpawnDynamicsCollider : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Optional solver - if set (and valid), it will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give the new collider component. Only used as a starting point - if another
	// component with this name already exists on the owner element, the hierarchy will append a
	// unique suffix. Read DynamicsColliderComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName ColliderComponentName = FRigDynamicsColliderComponent::GetDefaultName();

	// The collider shapes
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCollection Shapes;

	// The Dynamics Collider component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Exists", Keywords = "Query,Shape,Collision"))
struct FRigUnit_GetDynamicsColliderExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Collider
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Replaces the entire shape collection on a Dynamics collider. Construction-event only because the
// solver's parallel arrays are sized at Instantiate time and cannot grow or shrink at runtime; for
// editing existing shapes during simulation use the per-shape Set nodes (by Name) instead.
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Shapes", Varying))
struct FRigUnit_HierarchySetDynamicsColliderShapes : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The new shape collection (replaces the existing one wholesale)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCollection Shapes;
};

//======================================================================================================================
// Reads the entire shape collection from a Dynamics collider.
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Shapes", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderShapes : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The collider's shape collection. If the component is missing the output is left empty.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeCollection Shapes;
};

//======================================================================================================================
// Sets the properties of a named box shape on a Dynamics collider. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Box", Varying))
struct FRigUnit_HierarchySetDynamicsColliderBox : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The box shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeBox Box;
};

//======================================================================================================================
// Gets the properties of a named box shape on a Dynamics collider. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Box", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderBox : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The box shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeBox Box;
};

//======================================================================================================================
// Sets the properties of a named capsule shape on a Dynamics collider. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Capsule", Varying))
struct FRigUnit_HierarchySetDynamicsColliderCapsule : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The capsule shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCapsule Capsule;
};

//======================================================================================================================
// Gets the properties of a named capsule shape on a Dynamics collider. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Capsule", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderCapsule : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The capsule shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeCapsule Capsule;
};

//======================================================================================================================
// Sets the properties of a named plane shape on a Dynamics collider. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Plane", Varying))
struct FRigUnit_HierarchySetDynamicsColliderPlane : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The plane shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapePlane Plane;
};

//======================================================================================================================
// Gets the properties of a named plane shape on a Dynamics collider. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Plane", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderPlane : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The plane shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapePlane Plane;
};

//======================================================================================================================
// Sets the transform of a named box shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Box TM", Varying))
struct FRigUnit_HierarchySetDynamicsColliderBoxTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the box shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored)
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named box shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Box TM", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderBoxTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored). If no matching shape is found, the output is
	// left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the extents of a named box shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Box Extents", Varying))
struct FRigUnit_HierarchySetDynamicsColliderBoxExtents : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the box shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Full extents of the box in each axis
	UPROPERTY(meta = (Input))
	FVector Extents = FVector::OneVector * RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the extents of a named box shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Box Extents", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderBoxExtents : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Full extents of the box in each axis. If no matching shape is found, the output is left at
	// its default.
	UPROPERTY(meta = (Output))
	FVector Extents = FVector::OneVector * RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the transform of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Capsule TM", Varying))
struct FRigUnit_HierarchySetDynamicsColliderCapsuleTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored), main axis along +Z
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Capsule TM", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderCapsuleTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored), main axis along +Z. If no matching shape is
	// found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the radius of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Capsule Radius", Varying))
struct FRigUnit_HierarchySetDynamicsColliderCapsuleRadius : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Capsule radius
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Centimeters"))
	float Radius = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the radius of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Capsule Radius", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderCapsuleRadius : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Capsule radius. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float Radius = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the length of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Capsule Length", Varying))
struct FRigUnit_HierarchySetDynamicsColliderCapsuleLength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Core length of the capsule (total length is Length + 2 * Radius)
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Centimeters"))
	float Length = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the length of a named capsule shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Capsule Length", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderCapsuleLength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Core length of the capsule (total length is Length + 2 * Radius). If no matching shape is
	// found, the output is left at its default.
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float Length = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the transform of a named plane shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Plane TM", Varying))
struct FRigUnit_HierarchySetDynamicsColliderPlaneTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the plane shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position defines a point on the plane, plane faces +Z
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named plane shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Plane TM", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderPlaneTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position defines a point on the plane, plane faces +Z. If no matching shape is found, the
	// output is left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the extents of a named plane shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Set Dynamics Collider Plane Extents", Varying))
struct FRigUnit_HierarchySetDynamicsColliderPlaneExtents : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to be updated
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the plane shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Extents of the plane
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	FVector2D Extents = FVector2D(RigDynamicsDefaultShapeSize * 10.0f);
};

//======================================================================================================================
// Gets the extents of a named plane shape on a Dynamics collider
USTRUCT(meta = (DisplayName = "Get Dynamics Collider Plane Extents", Varying))
struct FRigUnit_HierarchyGetDynamicsColliderPlaneExtents : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Collider to query
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Extents of the plane. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FVector2D Extents = FVector2D(RigDynamicsDefaultShapeSize * 10.0f);
};

#undef UE_API
