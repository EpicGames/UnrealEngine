// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigParticleSimulation.h"

#include "RigDynamicsParticleComponent.h"

#include "RigDynamicsConeLimitComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// A component that can be added to a joint/element that defines a cone (angular) limit between three particles. 
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsConeLimitComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsConeLimitComponent)

	// The grandparent particle for the ConeLimit
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ConeLimit)
	FRigComponentKey GrandparentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The parent particle for the ConeLimit
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ConeLimit)
	FRigComponentKey ParentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The child particle for the ConeLimit
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ConeLimit)
	FRigComponentKey ChildComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength which we drive towards the target limit (when outside). This is the oscillation
	// frequency, so low values will be soft and springy, but values significantly above 1/timestep
	// will impose the limit strongly.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", Units = "Hertz"))
	float Strength = 10.0f;

	// DampingRatio for the ConeLimit
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0"))
	float DampingRatio = 1.0f;

	// The (full) cone angle in degrees
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", Units = "Degrees"))
	float Angle = 0.0f;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsConeLimit"); }

protected:
	void Serialize(FArchive& Ar);
};

#undef UE_API
