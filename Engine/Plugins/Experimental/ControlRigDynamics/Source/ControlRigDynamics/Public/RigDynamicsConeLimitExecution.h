// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsConeLimitComponent.h"
#include "RigDynamicsSolverComponent.h"
#include "RigDynamicsParticleComponent.h"

#include "RigParticleSimulation.h"

#include "RigDynamicsConeLimitExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics cone limit component.
// Note: This node only runs as part of the construction event.
//======================================================================================================================
USTRUCT(meta = (DisplayName = "Spawn Dynamics Cone Limit", Keywords = "Add,Construction,Create,Angle,Angular", Varying))
struct FRigUnit_SpawnDynamicsConeLimit : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Optional solver - if set, it will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give the new cone limit component. Only used as a starting point - if another
	// component with this name already exists on the owner element, the hierarchy will append a
	// unique suffix. Read DynamicsConeLimitComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName ConeLimitComponentName = FRigDynamicsConeLimitComponent::GetDefaultName();

	// The grandparent particle for the cone limit
	UPROPERTY(meta = (Input))
	FRigComponentKey GrandparentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The parent particle for the cone limit
	UPROPERTY(meta = (Input))
	FRigComponentKey ParentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The child particle for the cone limit
	UPROPERTY(meta = (Input))
	FRigComponentKey ChildComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength which we drive towards the target limit (when outside). This is the oscillation
	// frequency, so low values will be soft and springy, but values significantly above 1/timestep
	// will impose the limit strongly.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 10.0f;

	// DampingRatio for the cone limit
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 1.0f;

	// The (full) cone angle in degrees
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Degrees"))
	float Angle = 0.0f;

	// The Dynamics Cone Limit component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Cone Limit
USTRUCT(meta = (DisplayName = "Get Dynamics Cone Limit Exists", Keywords = "Query,Angle,Angular"))
struct FRigUnit_GetDynamicsConeLimitExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Cone Limit
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Sets the strength of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Set Dynamics Cone Limit Strength", Varying))
struct FRigUnit_HierarchySetDynamicsConeLimitStrength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to be updated
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// The strength which enforces the limit (oscillation frequency)
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 10.0f;
};

//======================================================================================================================
// Gets the strength of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Get Dynamics Cone Limit Strength", Varying))
struct FRigUnit_HierarchyGetDynamicsConeLimitStrength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to query
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// The strength which enforces the limit (oscillation frequency)
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Strength = 0.0f;
};

//======================================================================================================================
// Sets the damping ratio of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Set Dynamics Cone Limit Damping Ratio", Varying))
struct FRigUnit_HierarchySetDynamicsConeLimitDampingRatio : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to be updated
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// Damping ratio for the cone limit
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 1.0f;
};

//======================================================================================================================
// Gets the damping ratio of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Get Dynamics Cone Limit Damping Ratio", Varying))
struct FRigUnit_HierarchyGetDynamicsConeLimitDampingRatio : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to query
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// Damping ratio for the cone limit
	UPROPERTY(meta = (Output))
	float DampingRatio = 0.0f;
};

//======================================================================================================================
// Sets the angle of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Set Dynamics Cone Limit Angle", Varying))
struct FRigUnit_HierarchySetDynamicsConeLimitAngle : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to be updated
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// The (full) cone angle in degrees
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Degrees"))
	float Angle = 0.0f;
};

//======================================================================================================================
// Gets the angle of a Dynamics cone limit
USTRUCT(meta = (DisplayName = "Get Dynamics Cone Limit Angle", Varying))
struct FRigUnit_HierarchyGetDynamicsConeLimitAngle : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Cone Limit to query
	UPROPERTY(meta = (Input, DisplayName = "Cone Limit Component Key"))
	FRigComponentKey DynamicsConeLimitComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConeLimitComponent::GetDefaultName());

	// The (full) cone angle in degrees
	UPROPERTY(meta = (Output, Units = "Degrees"))
	float Angle = 0.0f;
};


#undef UE_API
