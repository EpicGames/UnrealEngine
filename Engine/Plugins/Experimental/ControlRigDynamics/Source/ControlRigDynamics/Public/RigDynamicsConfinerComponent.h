// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyComponents.h"

#include "RigDynamicsData.h"

#include "RigDynamicsConfinerComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// A component that can be added to a joint/element that defines how a confinement shape can be
// attached to it. DynamicsParticles that opt in to a confiner will be kept inside its shapes
// (each shape is enforced independently).
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsConfinerComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsConfinerComponent)

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes, meta = (ShowOnlyInnerProperties))
	FRigDynamicsShapeCollection Shapes;

	// The strength with which particles are pushed back inside the confiner shapes (oscillation
	// frequency). High values confine rigidly, low values allow soft/springy escape before settling.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsConfiner"); }
};

#undef UE_API
