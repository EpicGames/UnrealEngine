// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsConfinerComponent.h"
#include "RigDynamicsSolverComponent.h"

#include "RigDynamicsConfinerExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics confiner as a component on the owner element. Confiners keep opted-in
// particles inside each of their shapes (intersection semantics). The confiner tracks the input
// pose, but does not track the simulated pose.
// Note: This node only runs as part of the construction event.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Confiner",
	Keywords = "Add,Construction,Create,New,Shape,Confinement,Enclosure", Varying))
struct FRigUnit_SpawnDynamicsConfiner : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Optional solver - if set (and valid), it will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give the confiner component. Only used as a starting point - if another component
	// with this name already exists on the owner element, the hierarchy will append a unique
	// suffix. Read DynamicsConfinerComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName ConfinerComponentName = FRigDynamicsConfinerComponent::GetDefaultName();

	// The confiner shapes
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCollection Shapes;

	// Soft strength (oscillation frequency in Hz) with which particles are pushed back inside
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;

	// Particles that should be confined by this confiner. Each listed particle has the new
	// confiner key appended to its Confiners opt-in list, so no separate Enable call is needed.
	// Missing/invalid keys warn and are skipped. Leave empty to wire particles up later.
	UPROPERTY(meta = (Input))
	TArray<FRigComponentKey> ConfinedParticles;

	// The Dynamics Confiner component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Exists", Keywords = "Query,Shape,Confinement,Enclosure"))
struct FRigUnit_GetDynamicsConfinerExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Confiner
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Replaces the entire shape collection on a Dynamics confiner. Construction-event only because the
// solver's parallel arrays are sized at Instantiate time and cannot grow or shrink at runtime; for
// editing existing shapes during simulation use the per-shape Set nodes (by Name) instead.
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Shapes", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerShapes : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The new shape collection (replaces the existing one wholesale)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCollection Shapes;
};

//======================================================================================================================
// Reads the entire shape collection from a Dynamics confiner.
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Shapes", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerShapes : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The confiner's shape collection. If the component is missing the output is left empty.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeCollection Shapes;
};

//======================================================================================================================
// Sets the properties of a named box shape on a Dynamics confiner. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Box", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerBox : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The box shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeBox Box;
};

//======================================================================================================================
// Gets the properties of a named box shape on a Dynamics confiner. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Box", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerBox : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The box shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeBox Box;
};

//======================================================================================================================
// Sets the properties of a named capsule shape on a Dynamics confiner. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Capsule", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerCapsule : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The capsule shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapeCapsule Capsule;
};

//======================================================================================================================
// Gets the properties of a named capsule shape on a Dynamics confiner. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Capsule", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerCapsule : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The capsule shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapeCapsule Capsule;
};

//======================================================================================================================
// Sets the properties of a named plane shape on a Dynamics confiner. Name is used to find the shape.
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Plane", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerPlane : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The plane shape data (Name is used to find the shape to update)
	UPROPERTY(meta = (Input))
	FRigDynamicsShapePlane Plane;
};

//======================================================================================================================
// Gets the properties of a named plane shape on a Dynamics confiner. The shape is looked up by Name.
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Plane", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerPlane : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// The plane shape data. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FRigDynamicsShapePlane Plane;
};

//======================================================================================================================
// Sets the transform of a named box shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Box TM", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerBoxTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the box shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored)
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named box shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Box TM", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerBoxTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored). If no matching shape is found, the output is
	// left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the extents of a named box shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Box Extents", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerBoxExtents : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the box shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Full extents of the box in each axis
	UPROPERTY(meta = (Input))
	FVector Extents = FVector::OneVector * RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the extents of a named box shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Box Extents", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerBoxExtents : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the box shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Full extents of the box in each axis. If no matching shape is found, the output is left at
	// its default.
	UPROPERTY(meta = (Output))
	FVector Extents = FVector::OneVector * RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the transform of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Capsule TM", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerCapsuleTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored), main axis along +Z
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Capsule TM", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerCapsuleTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position and orientation (scale is ignored), main axis along +Z. If no matching shape is
	// found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the radius of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Capsule Radius", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerCapsuleRadius : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Capsule radius
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Centimeters"))
	float Radius = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the radius of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Capsule Radius", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerCapsuleRadius : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Capsule radius. If no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float Radius = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the length of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Capsule Length", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerCapsuleLength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Core length of the capsule (total length is Length + 2 * Radius)
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Centimeters"))
	float Length = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Gets the length of a named capsule shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Capsule Length", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerCapsuleLength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the capsule shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Core length of the capsule (total length is Length + 2 * Radius). If no matching shape is
	// found, the output is left at its default.
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float Length = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
// Sets the transform of a named plane shape on a Dynamics confiner. Extents are used for
// visualization only - the confiner treats each plane as an infinite +Z half-space.
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Plane TM", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerPlaneTM : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the plane shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Position defines a point on the plane, plane faces +Z
	UPROPERTY(meta = (Input))
	FTransform TM;
};

//======================================================================================================================
// Gets the transform of a named plane shape on a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Plane TM", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerPlaneTM : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Position defines a point on the plane, plane faces +Z. If no matching shape is found, the
	// output is left at its default.
	UPROPERTY(meta = (Output))
	FTransform TM;
};

//======================================================================================================================
// Sets the extents of a named plane shape on a Dynamics confiner (visualization only)
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Plane Extents", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerPlaneExtents : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the plane shape to update
	UPROPERTY(meta = (Input))
	FName Name;

	// Plane extents (used for visualization only; the confiner treats the plane as infinite)
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	FVector2D Extents = FVector2D(RigDynamicsDefaultShapeSize * 10.0f);
};

//======================================================================================================================
// Gets the extents of a named plane shape on a Dynamics confiner (visualization only)
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Plane Extents", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerPlaneExtents : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// The name of the plane shape to query
	UPROPERTY(meta = (Input))
	FName Name;

	// Plane extents (used for visualization only; the confiner treats the plane as infinite). If
	// no matching shape is found, the output is left at its default.
	UPROPERTY(meta = (Output))
	FVector2D Extents = FVector2D(RigDynamicsDefaultShapeSize * 10.0f);
};

//======================================================================================================================
// Sets the strength (oscillation frequency) of a Dynamics confiner
USTRUCT(meta = (DisplayName = "Set Dynamics Confiner Strength", Varying))
struct FRigUnit_HierarchySetDynamicsConfinerStrength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to be updated
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// Confinement strength (oscillation frequency in Hz). Zero disables the confiner.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;
};

//======================================================================================================================
// Gets the strength (oscillation frequency) of a Dynamics confiner
USTRUCT(meta = (DisplayName = "Get Dynamics Confiner Strength", Varying))
struct FRigUnit_HierarchyGetDynamicsConfinerStrength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Confiner to query
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());

	// Confinement strength (oscillation frequency in Hz). Zero disables the confiner.
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Strength = 2.0f;
};

#undef UE_API
