// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigParticleSimulation.h"

#include "RigDynamicsParticleComponent.h"

#include "RigDynamicsConstraintComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// A component that can be added to a joint/element that defines a constraint between two particles. 
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsConstraintComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsConstraintComponent)

	// The parent particle for the constraint
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Constraint)
	FRigComponentKey ParentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The child particle for the constraint
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Constraint)
	FRigComponentKey ChildComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If the constraint is marked as "hard", then the system will try as hard as possible to
	// enforce it. This will also be a little cheaper than making a soft constraint.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Constraint)
	ERigParticleSimulationConstraintType ConstraintType = ERigParticleSimulationConstraintType::Hard;

	// The strength which we drive towards the target length. This is the oscillation frequency, so
	// low values will be soft and springy, but values significantly above 1/timestep will track
	// the target very accurately.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard", Units = "Hertz"))
	float Strength = 2.0f;

	// DampingRatio for the constraint
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard"))
	float DampingRatio = 1.0f;

	// Additional damping for the constraint
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard", Units = "Hertz"))
	float ExtraDamping = 0.0f;

	// For soft constraints: When true, Strength is a mass-independent natural frequency. When
	// false, the spring/damper acts as a true force, so heavier particles oscillate more slowly.
	// For hard constraints, this has no effect.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard"))
	bool bAccelerationMode = true;

	// The target length will be calculated automatically, multiplied by this
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0"))
	float LengthMultiplier = 1.0f;

	// This will be added on to the target length after the automatic/multiplication
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (Units = "Centimeters"))
	float ExtraLength = 0.0f;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsConstraint"); }

protected:
	void Serialize(FArchive& Ar);
};

#undef UE_API
