// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsConstraintComponent.h"
#include "RigDynamicsSolverComponent.h"
#include "RigDynamicsParticleComponent.h"

#include "RigParticleSimulation.h"

#include "RigDynamicsConstraintExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics constraint component.
// Note: This node only runs as part of the construction event.
//======================================================================================================================
USTRUCT(meta = (DisplayName = "Spawn Dynamics Constraint", Keywords = "Add,Construction,Create,New,Rod,Distance", Varying))
struct FRigUnit_SpawnDynamicsConstraint : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Optional solver - if set, it will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give the new constraint component. Only used as a starting point - if another
	// component with this name already exists on the owner element, the hierarchy will append a
	// unique suffix. Read DynamicsConstraintComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName ConstraintComponentName = FRigDynamicsConstraintComponent::GetDefaultName();

	// The parent particle for the constraint
	UPROPERTY(meta = (Input))
	FRigComponentKey ParentComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The child particle for the constraint
	UPROPERTY(meta = (Input))
	FRigComponentKey ChildComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If the constraint is marked as "hard", then the system will try as hard as possible to
	// enforce it. This will also be a little cheaper than making a soft constraint.
	UPROPERTY(meta = (Input))
	ERigParticleSimulationConstraintType ConstraintType = ERigParticleSimulationConstraintType::Hard;

	// The strength which we drive towards the target length. This is the oscillation frequency, so
	// low values will be soft and springy, but values significantly above 1/timestep will track
	// the target very accurately.
	UPROPERTY(meta = (Input, ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard", Units = "Hertz"))
	float Strength = 2.0f;

	// DampingRatio for the constraint
	UPROPERTY(meta = (Input, ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard"))
	float DampingRatio = 1.0f;

	// Additional damping for the constraint
	UPROPERTY(meta = (Input, ClampMin = "0.0", EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard", Units = "Hertz"))
	float ExtraDamping = 0.0f;

	// For soft constraints: When true, Strength is a mass-independent natural frequency. When
	// false, the spring/damper acts as a true force, so heavier particles oscillate more slowly.
	// For hard constraints, this has no effect.
	UPROPERTY(meta = (Input, EditCondition = "ConstraintType != ERigParticleSimulationConstraintType::Hard"))
	bool bAccelerationMode = true;

	// The target length will be calculated automatically, multiplied by this
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LengthMultiplier = 1.0f;

	// This will be added on to the target length after the automatic/multiplication
	UPROPERTY(meta = (Input))
	float ExtraLength = 0.0f;

	// The Dynamics Constraint component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Exists", Keywords = "Query,Rod,Distance"))
struct FRigUnit_GetDynamicsConstraintExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Constraint
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Sets the strength of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Strength", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintStrength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// The strength which drives towards the target length (oscillation frequency)
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;
};

//======================================================================================================================
// Gets the strength of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Strength", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintStrength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// The strength which drives towards the target length (oscillation frequency)
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Strength = 0.0f;
};

//======================================================================================================================
// Sets the damping ratio of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Damping Ratio", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintDampingRatio : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Damping ratio for the constraint
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 1.0f;
};

//======================================================================================================================
// Gets the damping ratio of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Damping Ratio", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintDampingRatio : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Damping ratio for the constraint
	UPROPERTY(meta = (Output))
	float DampingRatio = 0.0f;
};

//======================================================================================================================
// Sets the extra damping of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Extra Damping", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintExtraDamping : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Additional damping for the constraint
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float ExtraDamping = 0.0f;
};

//======================================================================================================================
// Gets the extra damping of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Extra Damping", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintExtraDamping : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Additional damping for the constraint
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float ExtraDamping = 0.0f;
};

//======================================================================================================================
// Sets the length multiplier of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Length Multiplier", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintLengthMultiplier : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// The target length will be calculated automatically, multiplied by this
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float LengthMultiplier = 1.0f;
};

//======================================================================================================================
// Gets the length multiplier of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Length Multiplier", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintLengthMultiplier : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// The target length will be calculated automatically, multiplied by this
	UPROPERTY(meta = (Output))
	float LengthMultiplier = 0.0f;
};

//======================================================================================================================
// Sets the extra length of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Extra Length", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintExtraLength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Added on to the target length after the automatic/multiplication
	UPROPERTY(meta = (Input, Units = "Centimeters"))
	float ExtraLength = 0.0f;
};

//======================================================================================================================
// Gets the extra length of a Dynamics constraint
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Extra Length", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintExtraLength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// Added on to the target length after the automatic/multiplication
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float ExtraLength = 0.0f;
};

//======================================================================================================================
// Sets whether a Dynamics constraint is in acceleration mode (mass-independent natural frequency)
// or force mode (heavier particles oscillate slower). No effect on Hard constraints.
USTRUCT(meta = (DisplayName = "Set Dynamics Constraint Acceleration Mode", Varying))
struct FRigUnit_HierarchySetDynamicsConstraintAccelerationMode : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to be updated
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// True (default) for mass-independent natural frequency; false for a true force-mode spring.
	UPROPERTY(meta = (Input))
	bool bAccelerationMode = true;
};

//======================================================================================================================
// Gets whether a Dynamics constraint is in acceleration mode.
USTRUCT(meta = (DisplayName = "Get Dynamics Constraint Acceleration Mode", Varying))
struct FRigUnit_HierarchyGetDynamicsConstraintAccelerationMode : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Constraint to query
	UPROPERTY(meta = (Input, DisplayName = "Constraint Component Key"))
	FRigComponentKey DynamicsConstraintComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConstraintComponent::GetDefaultName());

	// True for mass-independent natural frequency; false for a true force-mode spring.
	UPROPERTY(meta = (Output))
	bool bAccelerationMode = true;
};

#undef UE_API
