// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyComponents.h"

#include "PhysicsControlData.h"

#include "RigParticleSimulation.h"

#include "RigDynamicsData.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

constexpr float RigDynamicsDefaultShapeSize = 10.0f;
constexpr float RigDynamicsDefaultParticleRadius = 5.0f;
constexpr float RigDynamicsDefaultParticleMass = 1.0f;

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsShapeBox
{
	GENERATED_BODY()

	FRigDynamicsShapeBox() {}

	FRigDynamicsShapeBox(const FName InName, const FTransform& InTM, const FVector& InExtents)
		: Name(InName), TM(InTM), Extents(InExtents) {}

	// Shape name is optional/only used for identification
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape)
	FName Name;

	// Position and orientation (scale is ignored)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta=(MakeEditWidget=true))
	FTransform TM;

	// These are the full extents of the box in each axis
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta = (ClampMin = "0.0", Units = "Centimeters"))
	FVector Extents = FVector::OneVector * RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsShapeCapsule
{
	GENERATED_BODY()

	FRigDynamicsShapeCapsule() {}

	FRigDynamicsShapeCapsule(const FName InName, const FTransform& InTM, const float InRadius, const float InLength)
		: Name(InName), TM(InTM), Radius(InRadius), Length(InLength) {}

	// Shape name is optional/only used for identification
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape)
	FName Name;

	// Position and orientation (scale is ignored)
	// Main axis will be along +Z. 
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta=(MakeEditWidget=true))
	FTransform TM;

	// Capsule radius
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta = (ClampMin = "0.0", Units = "Centimeters"))
	float Radius = RigDynamicsDefaultShapeSize;

	// This is the length of the core part of the capsule. The total length will be Length + 2 * Radius
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta = (ClampMin = "0.0", Units = "Centimeters"))
	float Length = RigDynamicsDefaultShapeSize;
};

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsShapePlane
{
	GENERATED_BODY()

	FRigDynamicsShapePlane() {}

	FRigDynamicsShapePlane(const FName InName, const FTransform& InTM, const FVector2D& InExtents)
		: Name(InName), TM(InTM), Extents(InExtents) {}

	// Shape name is optional/only used for identification
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape)
	FName Name;

	// Position defines a point on the plane. Plane faces in the +ve Z direction
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta=(MakeEditWidget=true))
	FTransform TM;

	// Extents of the plane
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shape, meta = (ClampMin = "0.0", Units = "Centimeters"))
	FVector2D Extents = FVector2D(RigDynamicsDefaultShapeSize * 10.0f);
};

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsShapeCollection
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes)
	TArray<FRigDynamicsShapeBox> Boxes;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes)
	TArray<FRigDynamicsShapeCapsule> Capsules;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes)
	TArray<FRigDynamicsShapePlane> Planes;

	bool IsEmpty() const { return Boxes.IsEmpty() && Capsules.IsEmpty() && Planes.IsEmpty(); }
	int32 NumShapes() const { return Boxes.Num() + Capsules.Num() + Planes.Num(); }
};

//======================================================================================================================
// Specifies what space the simulation should run in.
UENUM()
enum class ERigDynamicsSimulationSpace : uint8
{
	/** Simulate in world space. Moving the skeletal mesh will generate velocity changes */
	World,
	/** 
	* Simulate in component space. Moving the entire skeletal mesh will have no effect on velocities 
	* unless this is explicitly enabled in the simulation space settings */
	Component,
	/**
	* Simulate in the space of the specified bone. Moving the entire skeletal mesh and/or the Bone
	* will have no effect on velocities unless this is explicitly enabled in the simulation space settings.
	*/
	SpaceBone,
};

//======================================================================================================================
// Configuration/settings for the dynamics solver
USTRUCT(BlueprintType)
struct FRigDynamicsSolverSettings
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	ERigDynamicsSimulationSpace SimulationSpace = ERigDynamicsSimulationSpace::Component;

	// The bone to use for the simulation space (only relevant if SimulationSpace is set to SpaceBone)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	FRigElementKey SpaceBone = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Gravity in world space. This can be scaled by each particle.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	FVector Gravity = FVector(0.0f, 0.0f, -981.0f);

	// The maximum timestep of any step.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0.0", Units = "Seconds"))
	float MaxTimeStep = 1.0f;

	// The maximum number of steps that the update can be divided into. If MaxNumSteps * MaxTimeStep
	// is less than the update time, the simulation will run "slow".
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0"))
	int32 MaxNumSteps = 4;

	// This is the total number of iterations, including iterating over the particle targets. This
	// can normally be 1, unless you want to track targets with very accurate spring behavior.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0"))
	int32 NumIterations = 1;

	// The number of constraint (distance and collision) sub-iterations. Along-chain constraints
	// will be one-way on the last iteration, so if this is 1 (which is often enough), particles in
	// a chain will never be able to affect their parents. Increasing will result in more realistic
	// behavior that comes with a performance cost.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0"))
	int32 NumConstraintSubIterations = 1;

	// If true, then the read-back from the solver is done by rotating bones to match the directions
	// between particles. If there is no branching, then this will result in the final bones
	// matching the simulated particles (depending on the skeletal bone constraints being
	// fulfilled). However, when there is branching, there can be some deviation.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	bool bReadBoneOrientations = true;

	// If true the bone positions (translation) will be updated based on the particles. This will
	// introduce stretching etc in the skeleton that is not there in the original animation, but it
	// should result in better matching of the simulation and the bone positions.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings)
	bool bReadBonePositions = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (InlineEditConditionToggle))
	bool bResetFromPosition = false;

	// If any object in the simulation exceeds this distance from the simulation origin, we will
	// reset the pose and velocity of all objects in the simulation. This is to detect problems/explosions.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0.0", EditCondition = "bResetFromPosition", Units = "Centimeters"))
	float PositionThresholdForReset = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (InlineEditConditionToggle))
	bool bResetFromKinematicSpeed = false;

	// If any kinematic object in the simulation exceeds this speed, we will reset the pose and velocities of
	// all objects in the simulation. This is to detect problems - for example if the target
	// animation has teleports/jumps etc.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0.0", EditCondition = "bResetFromKinematicSpeed", Units = "CentimetersPerSecond"))
	float KinematicSpeedThresholdForReset = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (InlineEditConditionToggle))
	bool bResetFromEvaluationInterval = false;

	// If the time between successive solver evaluations (measured via the rig's absolute time)
	// exceeds this value AND is also greater than the frame's delta time, the pose and velocities
	// of all objects in the simulation are reset. This detects cases where evaluation was genuinely
	// skipped - e.g. the rig stopped being ticked, or the editor was paused, without triggering
	// just because the frame's delta time was rather large.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverSettings, meta = (ClampMin = "0.0", EditCondition = "bResetFromEvaluationInterval", Units = "Seconds"))
	float EvaluationIntervalThresholdForReset = 0.1f;
};

//======================================================================================================================
// Air/ether drag settings applied to the whole simulation, due to the movement of the simulation
// space (typically as a result of the actor/component moving). These act on top of the per-particle
// Damping. They describe the air/ether medium that particles are dragged toward.
USTRUCT(BlueprintType)
struct FRigDynamicsSimulationDragSettings
{
	GENERATED_BODY()

	// Extra air/ether damping added to every dynamic particle's own Damping value. Use this when
	// you want the wind (ExternalLinearVelocity, turbulence) to push every particle in the
	// simulation, including ones that were authored with a per-particle Damping of zero. 0 disables
	// it (only the per-particle Damping matters); higher values make every particle damp toward the
	// air/ether more aggressively. Like the per-particle Damping, this is scaled by the particle's
	// bScaleDampingByInverseMass flag so the global baseline shares the particle's mass-scaling choice.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", Units = "Hertz"))
	float AdditionalDamping = 0.0f;

	// How much of the simulation frame's linear velocity contributes to the air/ether target that
	// particles are dragged toward. 0 means the sim-space linear motion does not affect drag. 1
	// means particles are pulled toward world-at-rest as the space moves.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LinearDragMultiplier = 1.0f;

	// How much of the simulation frame's angular velocity contributes to the air/ether target. As for
	// LinearDragMultiplier but for the rotational part (rotation induced velocity at each particle position).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AngularDragMultiplier = 1.0f;

	// Extra linear air/ether velocity in world space. Particles with non-zero Damping are pulled
	// along with it, giving a wind-like effect. Typical values are similar to the velocity of the
	// object or effect, and usually around or less than 1000 for characters/wind.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (Units = "cm/s"))
	FVector ExternalLinearVelocity = FVector::ZeroVector;

	// Extra angular air/ether velocity in world space, units deg/s. Produces a linear air/ether
	// velocity of (AngVel x R) at each particle's sim-space position, on top of ExternalLinearVelocity.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (Units = "deg/s"))
	FVector ExternalAngularVelocity = FVector::ZeroVector;

	// Per-axis magnitude of Perlin turbulence added to ExternalLinearVelocity. Units are the same
	// as velocity, so this is the approximate magnitude of the turbulence along each component axis.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (Units = "cm/s"))
	FVector ExternalTurbulenceVelocity = FVector::ZeroVector;
};

//======================================================================================================================
// Master gain on the pseudo-forces (Coriolis, centrifugal, Euler, linear) caused by movement of the
// simulation space itself (typically, movement of the actor/component). 0 disables inertial-frame
// coupling entirely. These properties will not affect air/ether drag effects.
USTRUCT(BlueprintType)
struct FRigDynamicsInertialForceSettings
{
	GENERATED_BODY()

	// Master multiplier applied on top of the per-term multipliers below. Use this to scale all four
	// inertial-frame effects together. Set to 0 to disable inertial effects entirely. 
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float Amount = 1.0f;

	// How strongly bodies react when the actor accelerates in a straight line - e.g. when the
	// character starts running, brakes hard, or jumps. 0 disables this response; 1 is full
	// strength. Multiplied with the overall Amount.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float LinearEulerAmount = 1.0f;

	// How strongly bodies react when the actor's rotation speed changes - e.g. when the character
	// starts spinning or comes to a stop. Distinct from Centrifugal, which is about sustained
	// rotation. 0 disables; 1 is full strength. Multiplied with the overall Amount.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float AngularEulerAmount = 1.0f;

	// Strength of the outward "fling" that bodies feel while the actor is rotating - this is what
	// makes hair and cloth swing wide on a spinning character. 0 disables; 1 is full strength.
	// Multiplied with the overall Amount.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float CentrifugalAmount = 1.0f;

	// Strength of the sideways deflection that bodies experience as they move within a rotating
	// actor. Usually a subtle effect - the least visually noticeable of the four. 0 disables;
	// 1 is full strength. Multiplied with the overall Amount.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float CoriolisAmount = 1.0f;
};

//======================================================================================================================
// How the simulation-space linear/angular velocity and acceleration are conditioned (vertical scale
// + clamps) before being passed to the inertial pseudo-forces and the air/ether drag effects.
// These do not affect teleport detection.
USTRUCT(BlueprintType)
struct FRigDynamicsSimulationSpaceMotion
{
	GENERATED_BODY()

	// Multiplier on the vertical (Z) component of linear velocity AND linear acceleration before
	// clamping. Usually from 0.0 to 1.0 to reduce the effects of jumping and crouching on the
	// simulation, but it can be higher than 1.0 if you need to exaggerate this motion.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	float VerticalMotionScale = 1.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearVelocity = false;

	// A clamp on the effective world-space velocity that is passed to the simulation. It is not
	// usually required to change this but you would reduce this to limit the effects of drag on the
	// bodies in the simulation, for example. Expected values would be somewhat less than the usual
	// velocities of your object which is commonly a few hundred for a character.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearVelocity", Units = "CentimetersPerSecond"))
	float MaxLinearVelocity = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularVelocity = false;

	// A clamp on the effective world-space angular velocity that is passed to the simulation. Units
	// are deg/s, so a value of 360 is one full rotation per second. You would reduce this (and
	// MaxAngularAcceleration) to limit how much bodies "fly out" when the actor spins on the spot.
	// Especially useful if you have characters that can rotate very quickly, but you do not want to
	// see simulated objects flung outwards excessively. Values around or less than 720 are typical
	// in that case.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularVelocity", Units = "DegreesPerSecond"))
	float MaxAngularVelocity = 720.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearAcceleration = false;

	// A clamp on the effective world-space acceleration that is passed to the simulation. Used to
	// stop the bodies of the simulation flying out when suddenly changing linear speed. Useful when
	// characters can change from stationary to running very quickly, to prevent simulated objects
	// from being flung backwards. A common value for a character might be a few thousands (e.g.
	// 10x gravity).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearAcceleration"))
	float MaxLinearAcceleration = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularAcceleration = false;

	// A clamp on the effective world-space angular acceleration that is passed to the simulation.
	// Units are deg/s/s. Similar effect to MaxAngularVelocity but related to bodies flying
	// out/backwards when the rotation speed suddenly changes. A typical value for a character might
	// be around 7200.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularAcceleration"))
	float MaxAngularAcceleration = 7200.0f;

	// Inertial pseudo-force properties. Uses the conditioned velocity/acceleration above and applies
	// Coriolis / centrifugal / Euler / linear forces, scaled by Amount.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	FRigDynamicsInertialForceSettings InertialForces;

	// Air/ether drag. Uses the conditioned linear and angular velocities above.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	FRigDynamicsSimulationDragSettings Drag;
};

//======================================================================================================================
// Teleport-detection thresholds based on the movement of the simulation space. These operate on raw
// deltas in the simulation-space transform (not on the animation pose itself). When any threshold
// is crossed the solver zeroes velocities/accelerations for the frame without resetting the
// simulation pose.
USTRUCT(BlueprintType)
struct FRigDynamicsTeleportDetectionSettings
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromPositionChange = true;

	// If the linear position change of the simulation space is above this threshold, the movement is treated as a teleport.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromPositionChange", Units = "Centimeters"))
	float PositionChangeThreshold = 100.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromOrientationChange = true;

	// If the angular change (degrees) of the simulation space is above this threshold, the movement is treated as a teleport.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromOrientationChange", Units = "Degrees"))
	float OrientationChangeThreshold = 30.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromLinearAcceleration = true;

	// If the linear acceleration of the simulation space is above this threshold, the movement is treated as a teleport.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromLinearAcceleration"))
	float LinearAccelerationThreshold = 100000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bFromAngularAcceleration = true;

	// If the angular acceleration (deg/s/s) of the simulation space is above this threshold, the movement is treated as a teleport.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bFromAngularAcceleration"))
	float AngularAccelerationThreshold = 100000.0f;
};

//======================================================================================================================
// Legacy combined struct retained only for the deprecated FRigUnit_SpawnDynamicsSolver pin layout.
// Replaced by FRigDynamicsSimulationSpaceMotion (with drag and inertial settings nested) and a
// peer FRigDynamicsTeleportDetectionSettings as of FRigDynamicsObjectVersion::SimulationSpaceRegrouping.
// Use ConvertLegacyDynamicsSimulationSpaceSettings to populate the new structs from this one.
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsSimulationSpaceSettings
{
	GENERATED_BODY()

	// Overall multiplier on the effects of simulation space movement on the simulation. Mapped onto
	// the new master FRigDynamicsInertialForceSettings::Amount during conversion.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float SpaceMovementAmount = 1.0f;

	// Multiplier on the components of vertical velocity and acceleration of the simulation space
	// that is passed to the simulation. Mapped onto FRigDynamicsSimulationSpaceMotion::VerticalMotionScale.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings)
	float VelocityScaleZ = 1.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearVelocity = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearVelocity", Units = "CentimetersPerSecond"))
	float MaxLinearVelocity = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularVelocity = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularVelocity", Units = "DegreesPerSecond"))
	float MaxAngularVelocity = 720.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampLinearAcceleration = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampLinearAcceleration"))
	float MaxLinearAcceleration = 10000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bClampAngularAcceleration = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bClampAngularAcceleration"))
	float MaxAngularAcceleration = 7200.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bTeleportFromLinearAcceleration = true;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bTeleportFromLinearAcceleration"))
	float LinearAccelerationThresholdForTeleport = 100000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bTeleportFromAngularAcceleration = true;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bTeleportFromAngularAcceleration"))
	float AngularAccelerationThresholdForTeleport = 100000.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bTeleportFromPositionChange = true;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bTeleportFromPositionChange", Units = "Centimeters"))
	float PositionChangeThresholdForTeleport = 100.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (InlineEditConditionToggle))
	bool bTeleportFromOrientationChange = true;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", EditCondition = "bTeleportFromOrientationChange", Units = "Degrees"))
	float OrientationChangeThresholdForTeleport = 30.0f;
};

//======================================================================================================================
// Translates the legacy combined struct (from the deprecated FRigUnit_SpawnDynamicsSolver pin
// layout) into the new SpaceMotion / TeleportDetection structs. SpaceMotion.Drag is left untouched -
// in the legacy layout drag fields lived on a separate FRigDynamicsSimulationDragSettings pin, so
// the caller is expected to assign that separately.
UE_API void ConvertLegacyDynamicsSimulationSpaceSettings(
	const FRigDynamicsSimulationSpaceSettings& In,
	FRigDynamicsSimulationSpaceMotion&         OutMotion,
	FRigDynamicsTeleportDetectionSettings&     OutTeleport);

//======================================================================================================================
// Selects which particle property to display numerically next to each particle in the debug draw.
UENUM()
enum class ERigDynamicsParticleValueDisplay : uint8
{
	None,
	Radius,
	Mass,
	GravityMultiplier,
	Strength,
	DampingRatio,
	ExtraDamping,
	Damping,
	TargetMode,
	AngleLimit,
	AngleLimitStrength,
};

//======================================================================================================================
// Visualization toggles for the rig's dynamics simulation. Enabling any bShow* flag activates debug
// drawing - which has significant runtime cost - so disable everything before profiling.
// Visualization can also be globally suppressed at runtime via the CVar ControlRig.Dynamics.AllowVisualization.
USTRUCT(BlueprintType)
struct FRigDynamicsVisualizationSettings
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization, meta = (ClampMin = "0.0"))
	float LineThickness = 1.0f;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization, meta = (ClampMin = "6"))
	int32 ShapeDetail = 16;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowParticles = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowColliders = false;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowConfiners = false;

	// Show the auto-generated skeletal distance constraints between each particle and its parent.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowSkeletalConstraints = false;

	// Show user-defined hard distance constraints.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowHardConstraints = false;

	// Show user-defined soft distance constraints.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowSoftConstraints = false;

	// Show the per-particle angle-limit cones (with green tint). Cone apex is at the parent
	// particle, axis along the target direction, length = current bone length. Pale green when the
	// simulated direction is within the limit, red-with-green-mixed-in when it has swung outside.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowAngleLimits = false;

	// Show the per-cone-limit cones (with blue tint). Cone apex is at the parent particle, axis
	// along the incoming bone direction (grandparent->parent), length = current parent->child
	// distance. Pale blue when the child direction is within the limit, red-with-blue-mixed-in when
	// it has swung outside.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	bool bShowConeLimits = false;

	// When not None, the selected particle property is drawn as text next to each particle.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Visualization)
	ERigDynamicsParticleValueDisplay ParticleValueDisplay = ERigDynamicsParticleValueDisplay::None;
};

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsParticleProperties
{
	GENERATED_BODY()

	// The radius of the particle, used for collision
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0", Units = "Centimeters"))
	float Radius = RigDynamicsDefaultParticleRadius;

	// The mass of the particle, affecting how it will respond to constraints (and external forces etc)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ClampMin = "0.0001", Units = "Kilograms"))
	float Mass = RigDynamicsDefaultParticleMass;

	// How the particle moves
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics)
	ERigParticleSimulationMovementType MovementType = ERigParticleSimulationMovementType::Simulated;

	// Scales the solver gravity
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics)
	float GravityMultiplier = 1.0f;

	// The strength which we drive towards the target (when simulated). This is the oscillation
	// frequency, so low values will be soft and springy, but values significantly above
	// 1/timestep will track the target very accurately.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control, meta = (ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;

	// DampingRatio for the tracking particle target
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control, meta = (ClampMin = "0.0"))
	float DampingRatio = 0.5f;

	// Additional damping for tracking the particle target
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control, meta = (ClampMin = "0.0", Units = "Hertz"))
	float ExtraDamping = 0.0f;

	// When true, Strength is a mass-independent natural frequency. When false, the
	// spring/damper acts as a true force, so heavier particles oscillate more slowly. Applies to
	// target tracking only - the per-particle angle limit is always acceleration-mode regardless of
	// this flag.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	bool bAccelerationMode = true;

	// If TargetVelocityInfluence = 1 then the target's velocity will be used for damping. If 0 then the
	// damping is in simulation space (so acting more like drag)
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control)
	float TargetVelocityInfluence = 1.0f;

	// 0 = SimSpace (child tracks absolute animation target), 1 = directional (child+parent track
	// animation direction). Values between blend the two modes.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Control, meta = (ClampMin = "0.0", ClampMax = "1.0"))
	float TargetMode = 0.5f;

	// Constrains this particle to align with the target direction from its parent, to within this
	// angle (in degrees). This effectively gives us a limit on the deviation from the target pose.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = AngleLimits, meta = (ClampMin = "0.0", Units = "Degrees"))
	float AngleLimit = 0.0f;

	// The strength of the angle limit constraint (oscillation frequency). High values impose the
	// limit more rigidly, low values allow soft/springy deviation before settling. Zero disables
	// the angle limit.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = AngleLimits, meta = (ClampMin = "0.0", Units = "Hertz"))
	float AngleLimitStrength = 0.0f;

	// Air/ether damping (like drag). Particle velocity in simulation space is relaxed toward the
	// local air/ether (world-at-rest, plus any simulation-space motion scaled by the solver's
	// linear / angular drag multipliers). With bScaleDampingByInverseMass=false, the relaxation
	// timescale is 1/Damping for any mass. With bScaleDampingByInverseMass=true, Damping is divided
	// by mass so lighter particles damp faster, making this behave more like drag.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Drag, meta = (DisplayName = "Damping/Drag", ClampMin = "0.0", Units = "Hertz"))
	float Damping = 0.0f;

	// When true, Damping is divided by mass (drag-like - light particles damp faster). When false,
	// Damping is mass-independent (same relaxation timescale for any mass). Matches the
	// FRigPhysicsDynamics::bScaleDampingByInverseMass flag in ControlRigPhysics.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Drag, meta = (DisplayName = "Treat damping as drag"))
	bool bScaleDampingByInverseMass = false;

	// If false, this particle ignores all colliders regardless of NoCollisionColliders.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Collision)
	bool bCollideWithColliders = true;

	// A list of colliders we shouldn't collide with
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Collision)
	TArray<FRigComponentKey> NoCollisionColliders;

	// A list of other particles we should collide with
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Collision)
	TArray<FRigComponentKey> CollisionParticles;

	// A list of confiners that will keep this particle inside their shapes (opt-in).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Collision)
	TArray<FRigComponentKey> Confiners;
};

//======================================================================================================================
// A single per-frame force record queued by the AddDynamicsParticleForce node and consumed by the
// solver in UpdatePreDynamics. Particles have no orientation or extents, so there is no torque or
// location offset; just a direction/magnitude, the space the direction is given in, and the
// interpretation (force / acceleration / impulse / velocity change).
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsParticleForce
{
	GENERATED_BODY()

	FRigDynamicsParticleForce() = default;

	FRigDynamicsParticleForce(
		const FVector& InForce, EPhysicsControlSpace InSpace, EPhysicsControlForceType InType)
		: Force(InForce), Space(InSpace), Type(InType) {}

	// Direction and magnitude. Interpretation (units etc) depends on Type.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Force)
	FVector Force = FVector::ZeroVector;

	// Frame the Force is expressed in. Body means the space of the particle's owning bone in the
	// input pose (animation pose at the time of the step, before simulation writeback).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Force)
	EPhysicsControlSpace Space = EPhysicsControlSpace::World;

	// How Force is interpreted: Force (kg cm/s^2), Acceleration (cm/s^2, mass-independent),
	// Impulse (kg cm/s, one-shot momentum change), VelocityChange (cm/s, one-shot, mass-independent).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Force)
	EPhysicsControlForceType Type = EPhysicsControlForceType::Force;
};

//======================================================================================================================
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsParticleProperties& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsTeleportDetectionSettings& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsInertialForceSettings& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsSimulationSpaceMotion& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsSimulationDragSettings& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsSolverSettings& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeCollection& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsShapePlane& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeCapsule& Data);
UE_API FArchive& operator <<(FArchive& Ar, FRigDynamicsShapeBox& Data);

#undef UE_API
