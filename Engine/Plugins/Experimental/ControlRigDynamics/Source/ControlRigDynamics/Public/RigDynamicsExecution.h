// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Execution/RigUnit_DynamicHierarchy.h"

#include "RigDynamicsExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Base struct for all mutable dynamics nodes
//======================================================================================================================
USTRUCT(meta = (Category = "RigDynamics", NodeColor = "1.0 0.9 0.0", Keywords = "Dynamics", DocumentationPolicy = "Strict"))
struct FRigUnit_DynamicsBaseMutable : public FRigUnitMutable
{
	GENERATED_BODY()
};

//======================================================================================================================
// Base struct for all non-mutable dynamics nodes
//======================================================================================================================
USTRUCT(meta = (Category = "RigDynamics", NodeColor = "1.0 0.9 0.0", Keywords = "Dynamics", DocumentationPolicy = "Strict"))
struct FRigUnit_DynamicsBase : public FRigUnit
{
	GENERATED_BODY()
};

#undef UE_API

