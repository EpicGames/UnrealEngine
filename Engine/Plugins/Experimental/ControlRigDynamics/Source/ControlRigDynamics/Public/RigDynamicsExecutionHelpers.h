// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsSolverComponent.h"
#include "RigDynamicsParticleComponent.h"
#include "RigDynamicsColliderComponent.h"

#include "Curves/CurveFloat.h"

#include "RigDynamicsExecutionHelpers.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsShapeCollectionWithKey
{
	GENERATED_BODY()

	// The bone that owns the colliders
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes)
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// The collider properties
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Shapes, meta = (ShowOnlyInnerProperties))
	FRigDynamicsShapeCollection Shapes;
};


//======================================================================================================================
// Multiplier curves applied to particle properties along each chain. Each curve is evaluated at
// the particle's normalized along-chain position (0 at the chain root, 1 at the leaf) and
// multiplies the corresponding field of ParticleProperties. Default is a flat line at 1.0 (no
// change).
USTRUCT(BlueprintType)
struct FParticleChainCurves
{
	GENERATED_BODY()

	FParticleChainCurves()
	{
		// Default each multiplier curve to a flat line at 1.0 across the 0..1 range.
		auto InitFlatOne = [](FRuntimeFloatCurve& Curve)
		{
			FRichCurve* C = Curve.GetRichCurve();
			C->AddKey(0.0f, 1.0f);
			C->AddKey(1.0f, 1.0f);
		};
		InitFlatOne(RadiusMultiplier);
		InitFlatOne(MassMultiplier);
		InitFlatOne(GravityMultiplier);
		InitFlatOne(StrengthMultiplier);
		InitFlatOne(DampingRatioMultiplier);
		InitFlatOne(ExtraDampingMultiplier);
		InitFlatOne(DampingMultiplier);
		InitFlatOne(TargetModeMultiplier);
		InitFlatOne(AngleLimitMultiplier);
		InitFlatOne(AngleLimitStrengthMultiplier);
	}

	// Multiplier on ParticleProperties.Radius along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Radius Multiplier"))
	FRuntimeFloatCurve RadiusMultiplier;

	// Multiplier on ParticleProperties.Mass along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Mass Multiplier"))
	FRuntimeFloatCurve MassMultiplier;

	// Multiplier on ParticleProperties.GravityMultiplier along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Gravity Multiplier"))
	FRuntimeFloatCurve GravityMultiplier;

	// Multiplier on ParticleProperties.Strength along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Strength Multiplier"))
	FRuntimeFloatCurve StrengthMultiplier;

	// Multiplier on ParticleProperties.DampingRatio along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Damping Ratio Multiplier"))
	FRuntimeFloatCurve DampingRatioMultiplier;

	// Multiplier on ParticleProperties.ExtraDamping along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Extra Damping Multiplier"))
	FRuntimeFloatCurve ExtraDampingMultiplier;

	// Multiplier on ParticleProperties.Damping along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Damping Multiplier"))
	FRuntimeFloatCurve DampingMultiplier;

	// Multiplier on ParticleProperties.TargetMode along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Target Mode Multiplier"))
	FRuntimeFloatCurve TargetModeMultiplier;

	// Multiplier on ParticleProperties.AngleLimit along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Angle Limit Multiplier"))
	FRuntimeFloatCurve AngleLimitMultiplier;

	// Multiplier on ParticleProperties.AngleLimitStrength along each chain.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ChainCurves,
		meta = (XAxisName = "Chain Position", YAxisName = "Angle Limit Strength Multiplier"))
	FRuntimeFloatCurve AngleLimitStrengthMultiplier;
};

//======================================================================================================================
// Bundle of settings used when a spawn helper needs to create a new solver. When the helper is
// pointed at an existing solver (the SolverElementKey/SolverComponentName already resolves to a
// component) these fields are ignored and the existing solver's settings are preserved.
USTRUCT(BlueprintType)
struct FRigDynamicsSolverCreationSettings
{
	GENERATED_BODY()

	// Solver settings (gravity, iteration counts, substepping, reset thresholds, ...).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverCreation)
	FRigDynamicsSolverSettings SolverSettings;

	// Simulation-space motion conditioning (vertical scale + clamps), with the inertial-force
	// gain and air/ether drag settings nested inside.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverCreation)
	FRigDynamicsSimulationSpaceMotion SpaceMotion;

	// Teleport-detection thresholds (operates on raw simulation-space deltas).
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = SolverCreation)
	FRigDynamicsTeleportDetectionSettings TeleportDetection;
};

//======================================================================================================================
// Spawns a new dynamics chain (or chains)
// Note: This node only runs as part of the construction event.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Chains", Keywords = "Add,Construction,Create,New,Simulation", Varying))
struct FRigUnit_SpawnDynamicsChains : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_SpawnDynamicsChains()
	{
		ParticleProperties.GravityMultiplier = 0.0f;
	}

	// Optional solver - if valid then particles (etc) that are created will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// An array of root bones. Each one of these will start a chain, which includes all its
	// children/descendants. If a bone is reachable from more than one root (e.g. one root is an
	// ancestor of another), it will only have a single particle created for it.
	UPROPERTY(meta = (Input, BoneName))
	TArray<FRigElementKey> RootBones;

	// An array of bones that will terminate the chain search. Terminator bones will (if part of a
	// chain) have particles created, but their children will not.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ChainTerminatorBones;

	// If this is empty, then particles will be created for every bone discovered by the chain search.
	// Otherwise, only bones that are also in this list will receive particles.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ElementIncludeMask;

	// If this is empty, then no discovered bones will be excluded. Otherwise, bones in this list
	// will not receive particles even when reached by the chain search.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ElementExcludeMask;

	// Additional bones to be included (processed after other exclusions). Duplicates (bones already
	// covered by RootBones or another ExtraBones entry) are ignored.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ExtraBones;

	// The name to give each particle component created by this node. The same base name is used on
	// every owner bone, so the hierarchy will append unique suffixes whenever a clash occurs.
	// ParticleComponentKeys is the authoritative list of actual names.
	UPROPERTY(meta = (Input))
	FName ParticleComponentName = FRigDynamicsParticleComponent::GetDefaultName();

	// Basic properties for all the particles that are created. Note that the curves can be used to
	// vary these properties along the chains.
	UPROPERTY(meta = (Input))
	FRigDynamicsParticleProperties ParticleProperties;

	// Multiplier curves applied along each chain (evaluated at the particle's normalized along-chain
	// position). Defaults to flat lines at 1.0 - no change from ParticleProperties.
	UPROPERTY(meta = (Input))
	FParticleChainCurves ChainCurves;

	// The name to give each collider component created by this node. The same base name is used on
	// every owner bone, so the hierarchy will append unique suffixes whenever a clash occurs.
	// ColliderComponentKeys is the authoritative list of actual names.
	UPROPERTY(meta = (Input))
	FName ColliderComponentName = FRigDynamicsColliderComponent::GetDefaultName();

	// Additional Colliders that are attached to bones and track the incoming animation
	UPROPERTY(meta = (Input))
	TArray<FRigDynamicsShapeCollectionWithKey> Colliders;

	// The particle component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> ParticleComponentKeys;

	// The collider component keys that were created
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> ColliderComponentKeys;
};

//======================================================================================================================
// Imports collision from a physics asset and assigns it as colliders in the dynamics system
USTRUCT(meta = (DisplayName = "Import Dynamics Colliders From Physics Asset", Keywords = "Construction,Create,New", Varying, Deprecated = "5.8"))
struct FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	// Solver to add the colliders to
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give each collider component created by this node. The same base name is used
	// for every imported body, so the hierarchy will append unique suffixes whenever a clash
	// occurs. DynamicsColliderComponentKeys is the authoritative list of actual names.
	UPROPERTY(meta = (Input))
	FName ColliderComponentName = FRigDynamicsColliderComponent::GetDefaultName();

	// The physics asset to import colliders from
	UPROPERTY(meta = (Input))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// If this is empty, then all bones with bodies in the physics asset will be created. Otherwise
	// only bodies that relate to the specified bones will be created.
	UPROPERTY(meta = (Input))
	TArray<FName> BoneMask;

	// The dynamics collider component keys that were created
	UPROPERTY(meta = (Output, DisplayName = "Collider Component Keys"))
	TArray<FRigComponentKey> DynamicsColliderComponentKeys;
};

//======================================================================================================================
// Imports collision from a physics asset and assigns it as colliders in the dynamics system
USTRUCT(meta = (DisplayName = "Import Dynamics Colliders From Physics Asset", Keywords = "Construction,Create,New", Varying))
struct FRigUnit_HierarchyImportDynamicsCollidersFromPhysicsAsset1 : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// Solver to add the colliders to
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give each collider component created by this node. The same base name is used
	// for every imported body, so the hierarchy will append unique suffixes whenever a clash
	// occurs. DynamicsColliderComponentKeys is the authoritative list of actual names.
	UPROPERTY(meta = (Input))
	FName ColliderComponentName = FRigDynamicsColliderComponent::GetDefaultName();

	// The physics asset to import colliders from
	UPROPERTY(meta = (Input))
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	// If this is empty, then all bones with bodies in the physics asset will be created. Otherwise,
	// only bodies that relate to the specified elements will be created.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ElementIncludeMask;

	// If this is empty, then no bodies will be excluded. Otherwise, bodies that relate to the
	// specified elements will not be created.
	UPROPERTY(meta = (Input))
	TArray<FRigElementKey> ElementExcludeMask;

	// If true, then elements that have particles associated with our solver will be excluded from
	// having colliders made.
	UPROPERTY(meta = (Input))
	bool bExcludeElementsWithParticles = true;

	// The dynamics collider component keys that were created
	UPROPERTY(meta = (Output, DisplayName = "Collider Component Keys"))
	TArray<FRigComponentKey> DynamicsColliderComponentKeys;
};

//======================================================================================================================
// Multiplier curves applied to a force-field's three contributions. Each curve is evaluated at the
// particle's normalised radial position inside the field (0 at the centre, 1 at the surface) and
// multiplies the corresponding magnitude. Default is a ramp down to zero at the edge of the force field.
USTRUCT(BlueprintType)
struct FRigDynamicsForceFieldCurves
{
	GENERATED_BODY()

	FRigDynamicsForceFieldCurves()
	{
		// Default each multiplier curve to ramp down from 1.0 to 0 across the 0..1 range.
		auto InitRamp = [](FRuntimeFloatCurve& Curve)
		{
			FRichCurve* C = Curve.GetRichCurve();
			C->AddKey(0.0f, 1.0f);
			C->AddKey(1.0f, 0.0f);
		};
		InitRamp(RadialMultiplier);
		InitRamp(LinearMultiplier);
		InitRamp(RotationalMultiplier);
	}

	// Multiplier on the radial force magnitude.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ForceFieldCurves,
		meta = (XAxisName = "Radius fraction", YAxisName = "Radial Force Multiplier"))
	FRuntimeFloatCurve RadialMultiplier;

	// Multiplier on the linear force magnitude.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ForceFieldCurves,
		meta = (XAxisName = "Radius fraction", YAxisName = "Linear Force Multiplier"))
	FRuntimeFloatCurve LinearMultiplier;

	// Multiplier on the rotational force magnitude.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = ForceFieldCurves,
		meta = (XAxisName = "Radius fraction", YAxisName = "Rotational Force Multiplier"))
	FRuntimeFloatCurve RotationalMultiplier;
};

//======================================================================================================================
// Specifies forces on Dynamics particles to be applied in the next simulation step, based on the
// particle's location relative to the field. Multiple calls within the same frame accumulate.
// Forces queued during a pass-through frame (the next StepDynamicsSolver has Alpha <= 0) are
// discarded. Particles have no orientation, so there is no torque or location-offset variant.
//
// The force (which can be specified as a force, acceleration, impulse or velocity change) is made
// up of a radial, linear and rotational contribution.
USTRUCT(meta = (DisplayName = "Add Dynamics Particle Force Field", Keywords = "Force,Impulse,Acceleration,Velocity", Varying))
struct FRigUnit_HierarchyAddDynamicsParticleForceField : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	FRigUnit_HierarchyAddDynamicsParticleForceField()
	{
		// Somewhat arbitrary size for the field (in cm)
		float FieldSize = 1000;
		FieldTransform.SetScale3D(FVector(FieldSize));
	}

	// The Solver to use
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The position, orientation and extents of the field shape. The scale specifies the radius (cm) in
	// each direction, so the field ends up being an ellipsoid shape.
	UPROPERTY(meta = (Input))
	FTransform FieldTransform;

	// The space in which the field is specified.
	UPROPERTY(meta = (Input))
	ERigDynamicsSimulationSpace Space = ERigDynamicsSimulationSpace::World;

	// How Force is interpreted: Force (kg cm/s^2), Acceleration (cm/s^2, mass-independent),
	// Impulse (kg cm/s, one-shot momentum change), VelocityChange (cm/s, one-shot, mass-independent).
	UPROPERTY(meta = (Input))
	EPhysicsControlForceType Type = EPhysicsControlForceType::Force;

	// The magnitude of the radial contribution
	UPROPERTY(meta = (Input))
	float RadialForce = 0.0f;

	// The direction of the linear contribution. Auto-normalised internally, so a non-unit vector
	// does not double-encode the magnitude.
	UPROPERTY(meta = (Input))
	FVector LinearForceDirection = FVector::XAxisVector;

	// The magnitude of the linear contribution
	UPROPERTY(meta = (Input))
	float LinearForce = 0.0f;

	// The axis around which the rotational contribution is applied. Only the direction is used;
	// magnitude comes from RotationalForce.
	UPROPERTY(meta = (Input))
	FVector RotationalForceAxis = FVector::ZAxisVector;

	// The magnitude of the rotational contribution
	UPROPERTY(meta = (Input))
	float RotationalForce = 0.0f;

	// Per-radius-fraction multiplier curves applied to each of the three contributions.
	UPROPERTY(meta = (Input))
	FRigDynamicsForceFieldCurves FieldCurves;

	// If true, the field draws its ellipsoid shape and an arrow per in-field particle showing
	// this field's force contribution. Subject to the ControlRig.Dynamics.AllowVisualization CVar.
	UPROPERTY(meta = (Input))
	bool bDrawDebug = false;

	// Multiplier on the debug arrow length. Force magnitudes (kg cm/s^2 for Force, kg cm/s for
	// Impulse, etc.) are unwieldy as world-space lengths; tune until the arrows are legible.
	UPROPERTY(meta = (Input))
	float DebugForceScale = 1.0f;
};

#undef UE_API
