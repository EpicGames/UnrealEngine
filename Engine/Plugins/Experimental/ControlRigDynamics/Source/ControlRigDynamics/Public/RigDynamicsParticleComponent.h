// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Rigs/RigHierarchyComponents.h"

#include "RigDynamicsData.h"

#include "RigDynamicsParticleComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// A component that can be added to a joint/element that defines how a dynamics particle can be
// attached to it.
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsParticleComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsParticleComponent)

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Dynamics, meta = (ShowOnlyInnerProperties))
	FRigDynamicsParticleProperties ParticleProperties;

	// Per-frame queue of forces appended by AddDynamicsParticleForce nodes during the rig's
	// forwards-execution event. Consumed by FRigDynamicsSolver::UpdatePreDynamics each step (or
	// cleared when a step is skipped via Alpha <= 0).
	UPROPERTY(Transient)
	TArray<FRigDynamicsParticleForce> PendingForces;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsParticle"); }

protected:
	void Serialize(FArchive& Ar);
};

#undef UE_API
