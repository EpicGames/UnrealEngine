// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsParticleComponent.h"
#include "RigDynamicsColliderComponent.h"
#include "RigDynamicsConfinerComponent.h"
#include "RigDynamicsSolverComponent.h"

#include "RigDynamicsParticleExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics particle as a component on the owner element.
// Note: This node only runs as part of the construction event.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Particle", Keywords = "Add,Construction,Create,New,Body", Varying))
struct FRigUnit_SpawnDynamicsParticle : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(TEXT(""), ERigElementType::Bone);

	// Optional solver - if set, it will be added to this solver component
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The name to give the new particle component. Only used as a starting point - if another
	// component with this name already exists on the owner element, the hierarchy will append a
	// unique suffix. Read DynamicsParticleComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName ParticleComponentName = FRigDynamicsParticleComponent::GetDefaultName();

	// Particle properties
	UPROPERTY(meta = (Input))
	FRigDynamicsParticleProperties ParticleProperties;

	// The Dynamics Particle component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Exists", Keywords = "Query,Body"))
struct FRigUnit_GetDynamicsParticleExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Particle
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Disables collision between a particle and a collider
USTRUCT(meta = (DisplayName = "Disable Dynamics Collision With Collider", Varying))
struct FRigUnit_HierarchyDisableDynamicsCollisionWithCollider : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());


	// The Collider to not collide with (which doesn't have to exist yet)
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());
};

//======================================================================================================================
// Allows collision between a particle and a specific collider, by removing it from the particle's
// NoCollisionColliders list. Silent no-op if the collider is not in the list. Note that
// bCollideWithColliders is still the master switch - allowing a specific collider does not
// override that.
USTRUCT(meta = (DisplayName = "Allow Dynamics Collision With Collider", Varying))
struct FRigUnit_HierarchyAllowDynamicsCollisionWithCollider : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());


	// The Collider to re-enable collision with (which doesn't have to exist yet)
	UPROPERTY(meta = (Input, DisplayName = "Collider Component Key"))
	FRigComponentKey DynamicsColliderComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsColliderComponent::GetDefaultName());
};

//======================================================================================================================
// Replaces the entire NoCollisionColliders list on a particle.
USTRUCT(meta = (DisplayName = "Set Dynamics Particle No-Collision Colliders", Varying))
struct FRigUnit_HierarchySetDynamicsParticleNoCollisionColliders : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The list of colliders that this particle should not collide with
	UPROPERTY(meta = (Input))
	TArray<FRigComponentKey> NoCollisionColliders;
};

//======================================================================================================================
// Returns the current NoCollisionColliders list on a particle.
USTRUCT(meta = (DisplayName = "Get Dynamics Particle No-Collision Colliders", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleNoCollisionColliders : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The list of colliders that this particle does not collide with
	UPROPERTY(meta = (Output))
	TArray<FRigComponentKey> NoCollisionColliders;
};

//======================================================================================================================
// Enables collision between a particle and another particle. The collision will actually be between
// segments, where the segment goes from a particle top its parent (if it has one), using the radius
// of the particles to define a capsule. TODO Note that implementation isn't yet complete, so 
// currently it IS only sphere-sphere.
USTRUCT(meta = (DisplayName = "Enable Dynamics Collision With Particle", Varying))
struct FRigUnit_HierarchyEnableDynamicsCollisionWithParticle : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());


	// The other particle to collide with (which doesn't have to exist yet). Collision will be
	// handled between segments going from each particle to its parent (if it has one). TODO Note
	// that implementation isn't yet complete, so currently it IS only sphere-sphere.
	UPROPERTY(meta = (Input, DisplayName = "Other Particle Component Key"))
	FRigComponentKey OtherDynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());
};

//======================================================================================================================
// Disables collision between a particle and another particle by removing each key from the other's
// CollisionParticles list. Silent no-op for either side if the particle is missing or the key is
// not present in its list.
USTRUCT(meta = (DisplayName = "Disable Dynamics Collision With Particle", Varying))
struct FRigUnit_HierarchyDisableDynamicsCollisionWithParticle : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());


	// The other particle to stop colliding with
	UPROPERTY(meta = (Input, DisplayName = "Other Particle Component Key"))
	FRigComponentKey OtherDynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());
};

//======================================================================================================================
// Opts a particle in to being confined by a confiner. The confiner will keep the particle inside
// each of its shapes.
USTRUCT(meta = (DisplayName = "Enable Dynamics Confinement With Confiner", Varying))
struct FRigUnit_HierarchyEnableDynamicsConfinementWithConfiner : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The Confiner that should confine the particle (it doesn't have to exist yet)
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());
};

//======================================================================================================================
// Opts a particle out of being confined by a confiner.
USTRUCT(meta = (DisplayName = "Disable Dynamics Confinement With Confiner", Varying))
struct FRigUnit_HierarchyDisableDynamicsConfinementWithConfiner : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated (which must exist)
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The Confiner to remove from this particle's confinement list
	UPROPERTY(meta = (Input, DisplayName = "Confiner Component Key"))
	FRigComponentKey DynamicsConfinerComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsConfinerComponent::GetDefaultName());
};

//======================================================================================================================
// Sets the mass of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Mass", Varying))
struct FRigUnit_HierarchySetDynamicsParticleMass : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The mass of the Dynamics Particle
	UPROPERTY(meta = (Input, Units = "Kilograms", ClampMin = "0.0001"))
	float Mass = RigDynamicsDefaultParticleMass;
};

//======================================================================================================================
// Gets the mass of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Mass", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleMass : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The mass of the Dynamics Particle
	UPROPERTY(meta = (Output, Units = "Kilograms"))
	float Mass = RigDynamicsDefaultParticleMass;
};

//======================================================================================================================
// Sets the radius of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Radius", Varying))
struct FRigUnit_HierarchySetDynamicsParticleRadius : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The radius of the Dynamics Particle, used for collision
	UPROPERTY(meta = (Input, Units = "Centimeters", ClampMin = "0.0"))
	float Radius = RigDynamicsDefaultParticleRadius;
};

//======================================================================================================================
// Gets the radius of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Radius", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleRadius : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The radius of the Dynamics Particle, used for collision
	UPROPERTY(meta = (Output, Units = "Centimeters"))
	float Radius = RigDynamicsDefaultParticleRadius;
};

//======================================================================================================================
// Sets the movement type of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Movement Type", Varying))
struct FRigUnit_HierarchySetDynamicsParticleMovementType : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// How the particle should move
	UPROPERTY(meta = (Input))
	ERigParticleSimulationMovementType MovementType = ERigParticleSimulationMovementType::Simulated;
};

//======================================================================================================================
// Gets the movement type of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Movement Type", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleMovementType : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// How the particle should move
	UPROPERTY(meta = (Output))
	ERigParticleSimulationMovementType MovementType = ERigParticleSimulationMovementType::Kinematic;
};

//======================================================================================================================
// Sets the gravity multiplier of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Gravity Multiplier", Varying))
struct FRigUnit_HierarchySetDynamicsParticleGravityMultiplier : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Scales the solver gravity
	UPROPERTY(meta = (Input))
	float GravityMultiplier = 1.0f;
};

//======================================================================================================================
// Gets the gravity multiplier of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Gravity Multiplier", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleGravityMultiplier : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Scales the solver gravity
	UPROPERTY(meta = (Output))
	float GravityMultiplier = 0.0f;
};

//======================================================================================================================
// Sets the strength of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Strength", Varying))
struct FRigUnit_HierarchySetDynamicsParticleStrength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength which drives towards the target (oscillation frequency)
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Strength = 2.0f;
};

//======================================================================================================================
// Gets the strength of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Strength", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleStrength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength which drives towards the target (oscillation frequency)
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Strength = 0.0f;
};

//======================================================================================================================
// Sets the damping ratio of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Damping Ratio", Varying))
struct FRigUnit_HierarchySetDynamicsParticleDampingRatio : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Damping ratio for the particle
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float DampingRatio = 0.5f;
};

//======================================================================================================================
// Gets the damping ratio of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Damping Ratio", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleDampingRatio : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Damping ratio for the particle
	UPROPERTY(meta = (Output))
	float DampingRatio = 0.0f;
};

//======================================================================================================================
// Sets the extra damping of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Extra Damping", Varying))
struct FRigUnit_HierarchySetDynamicsParticleExtraDamping : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Extra damping for the particle
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float ExtraDamping = 0.0f;
};

//======================================================================================================================
// Gets the extra damping of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Extra Damping", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleExtraDamping : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Extra damping for the particle
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float ExtraDamping = 0.0f;
};

//======================================================================================================================
// Sets the particle into "drag" mode. Writes the Drag value to ParticleProperties.Damping and
// forces bScaleDampingByInverseMass = true, so the effective damping rate is Drag * InvMass
// (heavier particles damp slower, resulting in the drag-like behaviour). Use Set Damping instead
// for a mass-independent rate.
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Drag", Varying))
struct FRigUnit_HierarchySetDynamicsParticleDrag : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Air/ether drag coefficient (1/time). Effective coupling rate = Drag * InvMass.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Drag = 0.0f;
};

//======================================================================================================================
// Reads the particle's air/ether drag value, converting from Damping if the particle is in
// mass-independent mode (Drag = Damping * Mass) so the output always carries the drag-mode
// interpretation regardless of how the particle was authored.
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Drag", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleDrag : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Air/ether drag coefficient (1/time). Effective coupling rate = Drag * InvMass.
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Drag = 0.0f;
};

//======================================================================================================================
// Sets the particle into "damping" mode: writes the Damping value to ParticleProperties.Damping and
// forces bScaleDampingByInverseMass = false, so the effective coupling rate is Damping directly
// (mass-independent - every particle has the same relaxation timescale). Use Set Drag instead for
// the drag-like, mass-scaled behaviour.
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Damping", Varying))
struct FRigUnit_HierarchySetDynamicsParticleDamping : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Air/ether damping rate (1/time). Effective coupling rate = Damping, regardless of mass.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float Damping = 0.0f;
};

//======================================================================================================================
// Reads the particle's air/ether damping value, converting from Drag if the particle is in drag
// mode (Damping = Damping_stored / Mass) so the output always carries the damping-mode
// interpretation regardless of how the particle was authored.
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Damping", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleDamping : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Air/ether damping rate (1/time). Effective coupling rate = Damping, regardless of mass.
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float Damping = 0.0f;
};

//======================================================================================================================
// Sets the target velocity influence of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Target Velocity Influence", Varying))
struct FRigUnit_HierarchySetDynamicsParticleTargetVelocityInfluence : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If 1 the target's velocity will be used for damping, if 0 damping is in sim space
	UPROPERTY(meta = (Input, ClampMin = "0.0"))
	float TargetVelocityInfluence = 1.0f;
};

//======================================================================================================================
// Gets the target velocity influence of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Target Velocity Influence", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleTargetVelocityInfluence : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If 1 the target's velocity will be used for damping, if 0 damping is in sim space
	UPROPERTY(meta = (Output))
	float TargetVelocityInfluence = 0.0f;
};

//======================================================================================================================
// Sets the target mode of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Target Mode", Varying))
struct FRigUnit_HierarchySetDynamicsParticleTargetMode : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// 0 = SimSpace, 1 = directional, values between blend the two modes
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float TargetMode = 0.5f;
};

//======================================================================================================================
// Gets the target mode of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Target Mode", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleTargetMode : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// 0 = SimSpace, 1 = directional, values between blend the two modes
	UPROPERTY(meta = (Output))
	float TargetMode = 0.5f;
};

//======================================================================================================================
// Sets the angle limit of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Angle Limit", Varying))
struct FRigUnit_HierarchySetDynamicsParticleAngleLimit : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// This applies a soft constraint to make this particle align with the target direction from its
	// parent, to within this angle (in degrees). This effectively gives us a limit on the deviation
	// from the target pose.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Degrees"))
	float AngleLimit = 0.0f;
};

//======================================================================================================================
// Gets the angle limit of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Angle Limit", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleAngleLimit : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// This applies a soft constraint to make this particle align with the target direction from its
	// parent, to within this angle (in degrees). This effectively gives us a limit on the deviation
	// from the target pose.
	UPROPERTY(meta = (Output, Units = "Degrees"))
	float AngleLimit = 0.0f;
};

//======================================================================================================================
// Sets the angle limit strength of a Dynamics particle
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Angle Limit Strength", Varying))
struct FRigUnit_HierarchySetDynamicsParticleAngleLimitStrength : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength of the angle limit constraint (oscillation frequency). High values impose the
	// limit more rigidly, low values allow soft/springy deviation before settling. Zero disables
	// the angle limit.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Hertz"))
	float AngleLimitStrength = 0.0f;
};

//======================================================================================================================
// Gets the angle limit strength of a Dynamics particle
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Angle Limit Strength", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleAngleLimitStrength : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// The strength of the angle limit constraint (oscillation frequency). High values impose the
	// limit more rigidly, low values allow soft/springy deviation before settling. Zero disables
	// the angle limit.
	UPROPERTY(meta = (Output, Units = "Hertz"))
	float AngleLimitStrength = 0.0f;
};

//======================================================================================================================
// Sets whether a Dynamics particle participates in collider contacts
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Collide With Colliders", Varying))
struct FRigUnit_HierarchySetDynamicsParticleCollideWithColliders : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If false, this particle ignores all colliders regardless of NoCollisionColliders.
	UPROPERTY(meta = (Input))
	bool bCollideWithColliders = true;
};

//======================================================================================================================
// Gets whether a Dynamics particle participates in collider contacts
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Collide With Colliders", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleCollideWithColliders : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// If false, this particle ignores all colliders regardless of NoCollisionColliders.
	UPROPERTY(meta = (Output))
	bool bCollideWithColliders = true;
};

//======================================================================================================================
// Sets whether the particle's target drive is interpreted in acceleration mode (mass-independent
// natural frequency) or force mode (heavier particles oscillate more slowly). Affects target
// tracking only - the per-particle angle limit is always acceleration-mode.
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Acceleration Mode", Varying))
struct FRigUnit_HierarchySetDynamicsParticleAccelerationMode : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// True (default) for mass-independent natural frequency; false for a true force-mode spring.
	UPROPERTY(meta = (Input))
	bool bAccelerationMode = true;
};

//======================================================================================================================
// Gets the acceleration-mode flag of a Dynamics particle.
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Acceleration Mode", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleAccelerationMode : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// True for mass-independent natural frequency; false for a true force-mode spring.
	UPROPERTY(meta = (Output))
	bool bAccelerationMode = true;
};

//======================================================================================================================
// Sets whether the particle's Damping is scaled by inverse mass (drag-like) or mass-independent.
USTRUCT(meta = (DisplayName = "Set Dynamics Particle Scale Damping By Inverse Mass", Varying))
struct FRigUnit_HierarchySetDynamicsParticleScaleDampingByInverseMass : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to be updated
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// True: Damping is divided by mass (drag-like). False (default): Damping is mass-independent.
	UPROPERTY(meta = (Input))
	bool bScaleDampingByInverseMass = false;
};

//======================================================================================================================
// Gets the scale-damping-by-inverse-mass flag of a Dynamics particle.
USTRUCT(meta = (DisplayName = "Get Dynamics Particle Scale Damping By Inverse Mass", Varying))
struct FRigUnit_HierarchyGetDynamicsParticleScaleDampingByInverseMass : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to query
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// True: Damping is divided by mass (drag-like). False: Damping is mass-independent.
	UPROPERTY(meta = (Output))
	bool bScaleDampingByInverseMass = false;
};

//======================================================================================================================
// Specifies a force on a Dynamics particle to be applied on the next simulation step. Multiple
// calls within the same frame accumulate. Forces queued during a pass-through frame (the next
// StepDynamicsSolver has Alpha <= 0) are discarded. Particles have no orientation, so there is no
// torque or location-offset variant.
USTRUCT(meta = (DisplayName = "Add Dynamics Particle Force", Keywords = "Force,Impulse,Acceleration,Velocity", Varying))
struct FRigUnit_HierarchyAddDynamicsParticleForce : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Particle to apply the force to.
	UPROPERTY(meta = (Input, DisplayName = "Particle Component Key"))
	FRigComponentKey DynamicsParticleComponentKey = FRigComponentKey(
		FRigElementKey(TEXT(""), ERigElementType::Bone), FRigDynamicsParticleComponent::GetDefaultName());

	// Direction and magnitude of the force, in the frame given by Space. Units depend on Type.
	UPROPERTY(meta = (Input))
	FVector Force = FVector::ZeroVector;

	// Frame the Force is expressed in. Body means the space of the particle's owning bone in the
	// input pose.
	UPROPERTY(meta = (Input))
	EPhysicsControlSpace Space = EPhysicsControlSpace::World;

	// How Force is interpreted: Force (kg cm/s^2), Acceleration (cm/s^2, mass-independent),
	// Impulse (kg cm/s, one-shot momentum change), VelocityChange (cm/s, one-shot, mass-independent).
	UPROPERTY(meta = (Input))
	EPhysicsControlForceType Type = EPhysicsControlForceType::Force;
};

#undef UE_API
