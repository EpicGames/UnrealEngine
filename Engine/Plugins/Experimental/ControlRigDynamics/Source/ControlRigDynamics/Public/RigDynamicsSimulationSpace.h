// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RigDynamicsSolverComponent.h"
#include "RigParticleSimulation.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// This represents the simulation space, specified in world space. It will be associated with
// functions that allow conversion between the spaces. 
struct FRigDynamicsSimulationSpaceState
{
public:
	// If DeltaTime > 0 then this does a full update, calculating new velocities etc. If DeltaTime
	// <= 0 then this just updates the current position and orientation - no changes to velocity etc.
	// TeleportDetection is used to detect discontinuous simulation-space movement; when one is
	// detected the newly-computed velocities/accelerations are zeroed and UpdatesSinceReset is
	// reset so the following frame's acceleration is also suppressed via bAccelerationsValid.
	void Update(
		const FRigDynamicsTeleportDetectionSettings& TeleportDetection,
		const FTransform& InComponentTM,
		const FTransform& InSimulationSpaceTM,
		float DeltaTime);

	// This sets the position/orientation, and resets everything else.
	void Reset(const FTransform& InComponentTM, const FTransform& InSimulationSpaceTM);

	// Calculates the simulation space motion data from the current state and SpaceMotion (which
	// holds the conditioning, the inertial-force amount, and the nested air/ether drag settings),
	// ready to be passed to RigParticleSimulation::Simulate.
	RigParticleSimulation::FSimulationSpaceMotion CalculateMotion(
		const FRigDynamicsSimulationSpaceMotion& SpaceMotion,
		double AbsoluteTime) const;

	// Convert a position from component space (where the hierarchy lives) into simulation
	// space (where particles live). Identity when SimulationSpace=Component.
	RigParticleSimulation::FSimVector ConvertComponentSpacePositionToSimSpace(const FVector& ComponentSpacePosition) const;

	// Convert a position from simulation space back to component space.
	FVector ConvertSimSpacePositionToComponentSpace(const RigParticleSimulation::FSimVector& SimSpacePosition) const;

	// Convert a transform from component space into simulation space (rotation + translation, no scale).
	RigParticleSimulation::FSimTransform ConvertComponentSpaceTransformToSimSpace(const FTransform& ComponentSpaceTransform) const;

	// Convert a transform from simulation space back to component space.
	FTransform ConvertSimSpaceTransformToComponentSpace(const RigParticleSimulation::FSimTransform& SimSpaceTransform) const;

	// Convert a direction/velocity vector from world space into simulation space.
	RigParticleSimulation::FSimVector ConvertWorldVectorToSimSpace(const FVector& WorldVector) const;

	// Convert a direction/velocity vector from component space into simulation space.
	RigParticleSimulation::FSimVector ConvertComponentSpaceVectorToSimSpace(const FVector& ComponentSpaceVector) const;

	// Convert a transform from world space into simulation space (rotation + translation, no scale).
	RigParticleSimulation::FSimTransform ConvertWorldSpaceTransformToSimSpace(const FTransform& WorldSpaceTransform) const;

	// World-space transform of the rig's component (component-to-world). Tracked by the most
	// recent Update() / Reset() call.
	const FTransform& GetComponentTM() const { return ComponentTM; }

	// World-space linear velocity of the simulation space as computed by the most recent Update().
	// Zero on a frame where a teleport was detected.
	const FVector& GetLinearVelocity() const { return LinearVelocity; }

	// World-space angular velocity (rad/s) of the simulation space as computed by the most recent Update().
	const FVector& GetAngularVelocity() const { return AngularVelocity; }

	// World-space linear acceleration of the simulation space as computed by the most recent Update().
	const FVector& GetLinearAcceleration() const { return LinearAcceleration; }

	// World-space angular acceleration (rad/s/s) of the simulation space as computed by the most recent Update().
	const FVector& GetAngularAcceleration() const { return AngularAcceleration; }

	// True if the most recent Update() call (with DeltaTime > 0) detected a teleport via the
	// thresholds on FRigDynamicsTeleportDetectionSettings. Updates with DeltaTime <= 0 leave this
	// value unchanged. Reset() clears it.
	bool WasTeleportDetectedInLastUpdate() const { return bTeleportDetectedInLastUpdate; }

private:

	// Called when the component/simulation TMs have been updated
	void RecomputeCompositeTransforms();

	// The world-space component (owner of the simulation) transform
	FTransform ComponentTM;

	// World-space linear velocity of the simulation space
	FVector LinearVelocity = FVector::ZeroVector;
	// World-space angular velocity of the simulation space
	FVector AngularVelocity = FVector::ZeroVector; // rad/s

	// World-space linear acceleration of the simulation space
	FVector LinearAcceleration = FVector::ZeroVector;
	// World-space angular acceleration of the simulation space
	FVector AngularAcceleration = FVector::ZeroVector; // rad/s/s

	// The world-space simulation space TM
	FTransform SimulationSpaceTM;
	FTransform PrevSimulationSpaceTM;
	FTransform PrevPrevSimulationSpaceTM;

	// Precomputed composite transforms so that each coordinate conversion is a single transform
	// operation instead of two. Recomputed whenever ComponentTM or SimulationSpaceTM changes.
	FTransform ComponentToSimSpaceTM;
	FTransform SimToComponentSpaceTM;

	// The time between SimulationSpaceTM and PrevSimulationSpaceTM
	float DeltaTime = 0.0f;
	// The time between PrevSimulationSpaceTM and PrevPrevSimulationSpaceTM
	float PrevDeltaTime = 0.0f;

	// When we are reset, then the position (and orientation) is correct
	// On the next update (with a non-zero delta time), the velocity is correct (acceleration will be zero)
	// On the next update, the acceleration will be correct
	int64 UpdatesSinceReset = 0;

	// Set by Update() when teleport thresholds are tripped; updated only when DeltaTime > 0.
	bool bTeleportDetectedInLastUpdate = false;
};

//======================================================================================================================
// Inline implementations
//======================================================================================================================

//======================================================================================================================
inline RigParticleSimulation::FSimVector FRigDynamicsSimulationSpaceState::ConvertComponentSpacePositionToSimSpace(
	const FVector& ComponentSpacePosition) const
{
	return RigParticleSimulation::FSimVector(
		ComponentToSimSpaceTM.TransformPositionNoScale(ComponentSpacePosition));
}

//======================================================================================================================
inline FVector FRigDynamicsSimulationSpaceState::ConvertSimSpacePositionToComponentSpace(
	const RigParticleSimulation::FSimVector& SimSpacePosition) const
{
	return SimToComponentSpaceTM.TransformPositionNoScale(FVector(SimSpacePosition));
}

//======================================================================================================================
inline RigParticleSimulation::FSimTransform FRigDynamicsSimulationSpaceState::ConvertComponentSpaceTransformToSimSpace(
	const FTransform& ComponentSpaceTransform) const
{
	FTransform Result = ComponentSpaceTransform * ComponentToSimSpaceTM;
	Result.SetScale3D(FVector::OneVector);
	return RigParticleSimulation::FSimTransform(Result);
}

//======================================================================================================================
inline FTransform FRigDynamicsSimulationSpaceState::ConvertSimSpaceTransformToComponentSpace(
	const RigParticleSimulation::FSimTransform& SimSpaceTransform) const
{
	FTransform Result = FTransform(SimSpaceTransform) * SimToComponentSpaceTM;
	Result.SetScale3D(FVector::OneVector);
	return Result;
}

//======================================================================================================================
inline RigParticleSimulation::FSimVector FRigDynamicsSimulationSpaceState::ConvertWorldVectorToSimSpace(
	const FVector& WorldVector) const
{
	return RigParticleSimulation::FSimVector(
		SimulationSpaceTM.InverseTransformVectorNoScale(WorldVector));
}

//======================================================================================================================
inline RigParticleSimulation::FSimVector FRigDynamicsSimulationSpaceState::ConvertComponentSpaceVectorToSimSpace(
	const FVector& ComponentSpaceVector) const
{
	return RigParticleSimulation::FSimVector(
		ComponentToSimSpaceTM.TransformVectorNoScale(ComponentSpaceVector));
}

//======================================================================================================================
inline RigParticleSimulation::FSimTransform FRigDynamicsSimulationSpaceState::ConvertWorldSpaceTransformToSimSpace(
	const FTransform& WorldSpaceTransform) const
{
	FTransform Result = WorldSpaceTransform * SimulationSpaceTM.Inverse();
	Result.SetScale3D(FVector::OneVector);
	return RigParticleSimulation::FSimTransform(Result);
}

#undef UE_API
