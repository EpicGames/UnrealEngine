// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsData.h"

#include "Rigs/RigHierarchyComponents.h"

#include "Templates/SharedPointer.h"

#include "RigDynamicsSolverComponent.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

struct FRigDynamicsSolver;

//======================================================================================================================
// A component that can be added to a joint/element that defines how a system of particles can be solved
//======================================================================================================================
USTRUCT(BlueprintType)
struct FRigDynamicsSolverComponent : public FRigBaseComponent
{
public:
	GENERATED_BODY()
	DECLARE_RIG_COMPONENT_METHODS(FRigDynamicsSolverComponent)

	// Particles can be dynamic or kinematic. Kinematic particles track the target bones. Dynamic
	// particles fall under gravity, respond to constraints etc. In addition, dynamic particles will
	// track the target bones using forces. They will also be connected in chains to maintain the
	// original bone lengths.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Components)
	TArray<FRigComponentKey> Particles;

	// Collision shapes attached to bones in the hierarchy
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Components)
	TArray<FRigComponentKey> Colliders;

	// Constraints apply forces to pairs of particles to maintain a target distance
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Components)
	TArray<FRigComponentKey> Constraints;

	// Cone limits apply forces to triples of particles to limit the angle between them
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Components)
	TArray<FRigComponentKey> ConeLimits;

	// Confinement shapes attached to bones in the hierarchy. Particles that opt in to a confiner
	// will be kept inside its shapes.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Components)
	TArray<FRigComponentKey> Confiners;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Solver)
	FRigDynamicsSolverSettings Settings;

	// Simulation-space motion conditioning + the two consumers (inertial pseudo-forces and drag)
	// nested inside it. The conditioned velocities/accelerations feed both consumers.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Solver)
	FRigDynamicsSimulationSpaceMotion SpaceMotion;

	// Teleport-detection thresholds. Independent of SpaceMotion - these read raw deltas.
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Solver)
	FRigDynamicsTeleportDetectionSettings TeleportDetection;

	UE_API virtual void Save(FArchive& Ar) override;
	UE_API virtual void Load(FArchive& Ar) override;

	virtual FName GetDefaultComponentName() const override { return GetDefaultName(); }
	static FName GetDefaultName() { return TEXT("DynamicsSolver"); }

	// This will make the internally owned solver if necessary, and return it
	FRigDynamicsSolver* GetDynamicsSolver() const;

protected:
	void Serialize(FArchive& Ar);

private:

	// We use a TSharedPtr here in anticipation of supporting world collisions (see Control Rig Physics).
	//
	// We make it mutable so that GetDynamicsSolver can be const - in practice the user shouldn't
	// have to worry.
	//
	// Finally, UE's reflection system wants to be able to use the assignment operator. If it wasn't
	// for that, we could just delete the copy operator etc and prevent duplicates containing a
	// pointer to the same DynamicsSolver. In practice, this won't ever happen for a "live"
	// component, so we're OK.
	mutable TSharedPtr<FRigDynamicsSolver> DynamicsSolverPtr;
};

#undef UE_API
