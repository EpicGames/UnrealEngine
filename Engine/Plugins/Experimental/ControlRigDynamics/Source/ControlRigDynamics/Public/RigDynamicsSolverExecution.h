// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RigDynamicsExecution.h"
#include "RigDynamicsSolverComponent.h"

#include "Rigs/RigHierarchyCache.h"

#include "RigDynamicsSolverExecution.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

//======================================================================================================================
// Spawns a new dynamics solver as a component on the owner element.
// Note: This node only runs as part of the construction event.
//
// Deprecated in favour of FRigUnit_SpawnDynamicsSolver1, which exposes the new SpaceMotion +
// TeleportDetection input pins separately. Existing graphs continue to work - the deprecated
// Execute body routes the legacy SimulationSpaceSettings into the new component members via
// ConvertLegacyDynamicsSimulationSpaceSettings; right-clicking the deprecated node offers a
// one-click upgrade to the new node via GetUpgradeInfo.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Solver", Keywords = "Add,Construction,Create,New,Simulation", Varying, Deprecated = "5.8"))
struct FRigUnit_SpawnDynamicsSolver : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	RIGVM_METHOD()
	UE_API virtual FRigVMStructUpgradeInfo GetUpgradeInfo() const override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(FName(TEXT("Root")), ERigElementType::Bone);

	// The name to give the new solver component. Only used as a starting point - if another component
	// with this name already exists on the owner element, the hierarchy will append a unique
	// suffix. Read DynamicsSolverComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName SolverComponentName = FRigDynamicsSolverComponent::GetDefaultName();

	// Solver settings
	UPROPERTY(meta = (Input, DisplayName = "Solver Settings"))
	FRigDynamicsSolverSettings Settings;

	// Legacy combined simulation-space settings - now split into SpaceMotion + TeleportDetection on
	// the replacement unit.
	UPROPERTY(meta = (Input))
	FRigDynamicsSimulationSpaceSettings SimulationSpaceSettings;

	// Air/ether drag settings (drag multipliers, external air/ether velocity, turbulence). On the
	// replacement unit these live nested inside SpaceMotion.Drag.
	UPROPERTY(meta = (Input))
	FRigDynamicsSimulationDragSettings DragSettings;

	// The solver component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey;
};

//======================================================================================================================
// Spawns a new dynamics solver as a component on the owner element.
// Note: This node only runs as part of the construction event.
USTRUCT(meta = (DisplayName = "Spawn Dynamics Solver", Keywords = "Add,Construction,Create,New,Simulation", Varying))
struct FRigUnit_SpawnDynamicsSolver1 : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The owner of the newly created component (must be set/valid)
	UPROPERTY(meta = (Input, BoneName))
	FRigElementKey Owner = FRigElementKey(FName(TEXT("Root")), ERigElementType::Bone);

	// The name to give the new solver component. Only used as a starting point - if another component
	// with this name already exists on the owner element, the hierarchy will append a unique
	// suffix. Read DynamicsSolverComponentKey to find the name that was actually assigned.
	UPROPERTY(meta = (Input))
	FName SolverComponentName = FRigDynamicsSolverComponent::GetDefaultName();

	// Solver settings
	UPROPERTY(meta = (Input))
	FRigDynamicsSolverSettings Settings;

	// Teleport-detection thresholds based on the movement of the simulation space. These operate on
	// raw deltas in the simulation-space transform (not on the animation pose itself). When any
	// threshold is crossed the solver zeroes velocities/accelerations for the frame without
	// resetting the simulation pose.
	UPROPERTY(meta = (Input))
	FRigDynamicsTeleportDetectionSettings TeleportDetection;

	// How the simulation-space linear/angular velocity and acceleration are conditioned (vertical scale
	// + clamps) before being passed to the inertial pseudo-forces and the air/ether drag effects.
	// These do not affect teleport detection.
	UPROPERTY(meta = (Input))
	FRigDynamicsSimulationSpaceMotion SpaceMotion;

	// The solver component key that was created
	UPROPERTY(meta = (Output, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey;
};

//======================================================================================================================
// Indicates whether the component exists and is a Dynamics Solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Exists", Keywords = "Query,Simulation"))
struct FRigUnit_GetDynamicsSolverExists : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The component key to check
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Whether the component exists and is a Dynamics Solver
	UPROPERTY(meta = (Output))
	bool bExists = false;
};

//======================================================================================================================
// Instantiates all the objects in the dynamics world. Some properties can't be modified after this happens.
// Note that it will happen automatically during the first simulation step if it hasn't been explicitly
// requested. Explicit instantiation allows the timing to be controlled, as allocations etc may cause some
// delays.
USTRUCT(meta = (DisplayName = "Instantiate dynamics"))
struct FRigUnit_InstantiateDynamics : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The solver that should be instantiated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());
};


//======================================================================================================================
// Steps the dynamics solver
USTRUCT(meta = (DisplayName = "Step Dynamics Solver", Keywords = "Simulate"))
struct FRigUnit_StepDynamicsSolver : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The physics solver that should be stepped
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// If this is zero, then the execute context time will be used. If this is positive then it will
	// override the delta time. A negative value will prevent the solver from stepping.
	UPROPERTY(meta = (Input, Units = "Seconds"))
	float DeltaTimeOverride = 0.0f;

	// If this is zero, then the simulation delta time will be used for evaluating movement of the
	// simulation space. If this is positive then it will override. This may be needed if the
	// component movement is being done in parallel, in which case you might need to pass in the
	// previous time delta here.
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Seconds"))
	float SimulationSpaceDeltaTimeOverride = 0.0f;

	// How much of the simulation is combined with the input bone. This currently happens in
	// component space. When alpha <= zero, the simulation is bypassed entirely (pass-through).
	UPROPERTY(meta = (Input, ClampMin = "0.0", ClampMax = "1.0"))
	float Alpha = 1.0f;

	// When true and Alpha is zero, the solver tracks particle positions each frame so that
	// velocities are valid when Alpha increases (smooth resume). This has a small cost. When false
	// and Alpha is zero, the pass-through is near-zero cost but the simulation re-initializes with
	// zero velocity when Alpha increases.
	UPROPERTY(meta = (Input))
	bool bTrackVelocitiesDuringPassThrough = true;

	// Settings that specify how the solver state should be visualized during/after the step
	UPROPERTY(meta = (Input))
	FRigDynamicsVisualizationSettings VisualizationSettings;

	// Doing a lookup to get the SolverComponent from the key is slow, so cache this. If it
	// matches DynamicsSolverComponentKey we can use the cached value.
	UPROPERTY()
	FCachedRigComponent CachedSolverComponent;
};

//======================================================================================================================
// Sets the complete settings of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Settings", Varying))
struct FRigUnit_HierarchySetDynamicsSolverSettings : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The solver settings
	UPROPERTY(meta = (Input))
	FRigDynamicsSolverSettings Settings;
};

//======================================================================================================================
// Gets the complete settings of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Settings", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverSettings : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The solver settings
	UPROPERTY(meta = (Output))
	FRigDynamicsSolverSettings Settings;
};

//======================================================================================================================
// Sets the gravity of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Gravity", Varying))
struct FRigUnit_HierarchySetDynamicsSolverGravity : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Gravity in world space, can be scaled per particle
	UPROPERTY(meta = (Input))
	FVector Gravity = FVector(0.0f, 0.0f, -981.0f);
};

//======================================================================================================================
// Gets the gravity of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Gravity", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverGravity : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Gravity in world space, can be scaled per particle
	UPROPERTY(meta = (Output))
	FVector Gravity = FVector::ZeroVector;
};

//======================================================================================================================
// Sets the max time step of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Max Time Step", Varying))
struct FRigUnit_HierarchySetDynamicsSolverMaxTimeStep : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The maximum timestep of any step
	UPROPERTY(meta = (Input, ClampMin = "0.0", Units = "Seconds"))
	float MaxTimeStep = 1.0f;
};

//======================================================================================================================
// Gets the max time step of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Max Time Step", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverMaxTimeStep : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The maximum timestep of any step
	UPROPERTY(meta = (Output, Units = "Seconds"))
	float MaxTimeStep = 0.0f;
};

//======================================================================================================================
// Sets the max number of steps of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Max Num Steps", Varying))
struct FRigUnit_HierarchySetDynamicsSolverMaxNumSteps : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The maximum number of steps per update
	UPROPERTY(meta = (Input, ClampMin = "0"))
	int32 MaxNumSteps = 1;
};

//======================================================================================================================
// Gets the max number of steps of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Max Num Steps", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverMaxNumSteps : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The maximum number of steps per update
	UPROPERTY(meta = (Output))
	int32 MaxNumSteps = 0;
};

//======================================================================================================================
// Sets the number of iterations of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Num Iterations", Varying))
struct FRigUnit_HierarchySetDynamicsSolverNumIterations : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Total number of iterations including particle target tracking
	UPROPERTY(meta = (Input, ClampMin = "0"))
	int32 NumIterations = 1;
};

//======================================================================================================================
// Gets the number of iterations of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Num Iterations", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverNumIterations : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Total number of iterations including particle target tracking
	UPROPERTY(meta = (Output))
	int32 NumIterations = 0;
};

//======================================================================================================================
// Sets the number of constraint sub-iterations of a Dynamics solver
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Num Constraint Sub-Iterations", Varying))
struct FRigUnit_HierarchySetDynamicsSolverNumConstraintSubIterations : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Number of constraint (distance and collision) sub-iterations
	UPROPERTY(meta = (Input, ClampMin = "0"))
	int32 NumConstraintSubIterations = 1;
};

//======================================================================================================================
// Gets the number of constraint sub-iterations of a Dynamics solver
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Num Constraint Sub-Iterations", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverNumConstraintSubIterations : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Number of constraint (distance and collision) sub-iterations
	UPROPERTY(meta = (Output))
	int32 NumConstraintSubIterations = 0;
};

//======================================================================================================================
// Sets the simulation-space motion settings (conditioning + nested inertial-force / drag) of a
// Dynamics solver.
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Space Motion", Varying))
struct FRigUnit_HierarchySetDynamicsSolverSpaceMotion : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The simulation-space motion conditioning, including the nested inertial-force gain and drag settings.
	UPROPERTY(meta = (Input))
	FRigDynamicsSimulationSpaceMotion SpaceMotion;
};

//======================================================================================================================
// Gets the simulation-space motion settings of a Dynamics solver.
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Space Motion", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverSpaceMotion : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The simulation-space motion conditioning, including the nested inertial-force gain and drag settings.
	UPROPERTY(meta = (Output))
	FRigDynamicsSimulationSpaceMotion SpaceMotion;
};

//======================================================================================================================
// Sets the teleport-detection thresholds of a Dynamics solver.
USTRUCT(meta = (DisplayName = "Set Dynamics Solver Teleport Detection", Varying))
struct FRigUnit_HierarchySetDynamicsSolverTeleportDetection : public FRigUnit_DynamicsBaseMutable
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to be updated
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Per-channel teleport-detection thresholds applied to raw simulation-space deltas.
	UPROPERTY(meta = (Input))
	FRigDynamicsTeleportDetectionSettings TeleportDetection;
};

//======================================================================================================================
// Gets the teleport-detection thresholds of a Dynamics solver.
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Teleport Detection", Varying))
struct FRigUnit_HierarchyGetDynamicsSolverTeleportDetection : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The Dynamics Solver to query
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// Per-channel teleport-detection thresholds applied to raw simulation-space deltas.
	UPROPERTY(meta = (Output))
	FRigDynamicsTeleportDetectionSettings TeleportDetection;
};

//======================================================================================================================
// Retrieves the data that were generated during the simulation step, so the values returned will
// relate to the previous update if the solver has not yet been stepped.
USTRUCT(meta = (DisplayName = "Get Dynamics Solver Data", Keywords = "Debug"))
struct FRigUnit_GetDynamicsSolverData : public FRigUnit_DynamicsBase
{
	GENERATED_BODY()

	RIGVM_METHOD()
	UE_API virtual void Execute() override;

	// The solver from which we'll get these data
	UPROPERTY(meta = (Input, DisplayName = "Solver Component Key"))
	FRigComponentKey DynamicsSolverComponentKey = FRigComponentKey(
		FRigElementKey(TEXT("Root"), ERigElementType::Bone), FRigDynamicsSolverComponent::GetDefaultName());

	// The linear velocity of the simulation space, in world space (cm/s).
	UPROPERTY(meta = (Output))
	FVector SimulationSpaceLinearVelocity = FVector::ZeroVector;

	// The angular velocity of the simulation space, in world space (deg/s).
	UPROPERTY(meta = (Output))
	FVector SimulationSpaceAngularVelocity = FVector::ZeroVector;

	// The linear acceleration of the simulation space, in world space (cm/s/s).
	UPROPERTY(meta = (Output))
	FVector SimulationSpaceLinearAcceleration = FVector::ZeroVector;

	// The angular acceleration of the simulation space, in world space (deg/s/s).
	UPROPERTY(meta = (Output))
	FVector SimulationSpaceAngularAcceleration = FVector::ZeroVector;

	// Whether or not a teleport was detected
	UPROPERTY(meta = (Output))
	bool bTeleportDetected = false;

	// True if the most recent step reset all particle velocities because a kinematic particle's
	// sim-space target speed exceeded KinematicSpeedThresholdForReset.
	UPROPERTY(meta = (Output))
	bool bKinematicSpeedResetTriggered = false;

	// True if the most recent step snapped the simulation back to the animation pose because a
	// particle's sim-space distance from the origin exceeded PositionThresholdForReset.
	UPROPERTY(meta = (Output))
	bool bPositionResetTriggered = false;

	// True if the most recent step snapped the simulation back to the animation pose because the
	// interval between successive evaluations exceeded EvaluationIntervalThresholdForReset.
	UPROPERTY(meta = (Output))
	bool bEvaluationIntervalResetTriggered = false;
};


#undef UE_API
