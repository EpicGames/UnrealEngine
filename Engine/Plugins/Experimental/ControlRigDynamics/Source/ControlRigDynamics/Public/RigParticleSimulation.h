// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RigParticleSimulation.generated.h"

#define UE_API CONTROLRIGDYNAMICS_API

// Uncomment to use single-precision (float) vectors/transforms in the simulation. By default the
// simulation uses the engine's FVector/FTransform (double).
//
// However, note that profiling indicates only a small improvement (5%) through using floats, even
// when multiple iterations/complex setups are used.

//  #define CONTROLRIGDYNAMICS_SINGLEPRECISION

//======================================================================================================================
// This should be kept completely independent of control rig etc
//======================================================================================================================

//======================================================================================================================
UENUM(BlueprintType)
enum class ERigParticleSimulationMovementType : uint8
{
	// Kinematic means that the particle will track animation
	Kinematic,
	// Simulated means that the particle will be controlled by dynamics
	Simulated
};

//======================================================================================================================
// TODO support min, max and ranged length constraints
UENUM(BlueprintType)
enum class ERigParticleSimulationConstraintType : uint8
{
	// The target distance will be enforced as hard as possible
	Hard,
	// The target distance will be enforced softly, based on the strengths
	Soft
};

namespace RigParticleSimulation
{

#ifdef CONTROLRIGDYNAMICS_SINGLEPRECISION
using FSimVector    = FVector3f;
using FSimTransform = FTransform3f;
using FSimVector2D  = FVector2f;
using FSimScalar    = float;
#else
using FSimVector    = FVector;
using FSimTransform = FTransform;
using FSimVector2D  = FVector2D;
using FSimScalar    = double;
#endif

//======================================================================================================================
// Simulation data used in the central loops of the solve.
// 
// Note that each one of these has an Update function, which may need to be called before the
// simulate call. The user should only modify things through this update (after instantiation).
//======================================================================================================================

//======================================================================================================================
// We have an "Info" structure that accompanies the simulation structure. The Info contains data
// that is "user-facing" (the user is control rig), and data that is helpful for updating the
// simulation structures.
struct FParticleInfo
{
	// Index into the Particles/ParticleInfos arrays to retrieve the parent particle
	int32 ParentParticleIndex = INDEX_NONE;
	int32 NumChildren = 0;

	// True when the parent particle's bone is the direct skeleton parent
	// of this particle's bone (no intermediate bones skipped). Allows
	// computing child global position from parent transform + child local.
	bool bParentParticleIsDirectParent = false;

	// The previous/current update targets
	FSimVector PreviousTargetPosition = FSimVector::ZeroVector;
	FSimVector TargetPosition = FSimVector::ZeroVector;
	FSimVector PreviousTargetVelocity = FSimVector::ZeroVector;
	FSimVector TargetVelocity = FSimVector::ZeroVector;

	// Normalized direction from parent particle to this particle in the target/animation pose
	FSimVector TargetDirectionFromParent = FSimVector::ZeroVector;
	FSimVector PreviousTargetDirectionFromParent = FSimVector::ZeroVector;

	float GravityMultiplier = 1.0f;
	FSimVector ExternalForce = FSimVector::ZeroVector;
	ERigParticleSimulationMovementType MovementType = ERigParticleSimulationMovementType::Simulated;

	// Air/ether drag rate (1/time). Zero disables per-particle drag. See UpdateParticle.
	float Damping = 0.0f;

	// When true, Damping is scaled by InvMass (drag-like - lighter particles damp faster).
	// When false, Damping is mass-independent (same relaxation timescale for any mass).
	bool bScaleDampingByInverseMass = false;
};

//======================================================================================================================
struct FParticle
{
	float InvMass = 0.0f; // 0 means kinematic (and track the target)

	FSimVector Position = FSimVector::ZeroVector;
	FSimVector PrevPosition = FSimVector::ZeroVector;

	// The target position is used for kinematic motion
	FSimVector TargetPosition = FSimVector::ZeroVector;
	FSimVector TargetVelocity = FSimVector::ZeroVector;
};

//======================================================================================================================
struct FParticleTarget
{
	// How we drive to the target
	float Compliance = 0.0f; // alpha
	float Damping = 0.0f;    // beta
	float TargetVelocityInfluence = 0.0f;
	// 0 = SimSpace (child tracks absolute target), 1 = directional (child+parent track
	// animation direction). Values between blend the two modes.
	float TargetMode = 0.5f;

	// The current interpolated direction from parent to child (unit vector, per substep)
	FSimVector TargetDirectionFromParent = FSimVector::ZeroVector;

	// Angle limit. Precomputed cos/sin so the solve loop avoids trig calls.
	// AngleLimitCompliance zero means disabled.
	float CosAngleLimit = 1.0f;
	float SinAngleLimit = 0.0f;
	float AngleLimitCompliance = 0.0f;

	// Accumulated
	FSimVector Lambda = FSimVector::ZeroVector;
};

//======================================================================================================================
struct FConeLimitInfo
{
	int32 GrandparentIndex = -1;
	int32 ParentIndex = -1;
	int32 ChildIndex = -1;
};

//======================================================================================================================
struct FConeLimit
{
	float Angle = 0.0f;      // Radians
	float Compliance = 0.0f; // alpha
	float Damping = 0.0f;    // beta
	// Accumulated
	float Lambda = 0.0f;
};

//======================================================================================================================
struct FParticleCollider
{
	float Radius = 0.0f;
	bool bCollideWithColliders = true;
};

//======================================================================================================================
struct FParticleNoCollision
{
	// Flag per simulation state colliders indicating whether they should collide with this particle
	TBitArray<> NoCollisionColliderIndices;
};

//======================================================================================================================
struct FParticleToParticleCollision
{
	// Indices (into the simulation state particles) which this particle will collide with
	TArray<int32> CollisionParticleIndices;
};

//======================================================================================================================
struct FParticleConfinement
{
	// Indices (into the simulation state confiners) which keep this particle inside their shapes.
	// Each listed confiner is enforced independently (intersection semantics).
	TArray<int32> ConfinerIndices;
};

//======================================================================================================================
struct FHardDistanceConstraintInfo
{
	int32 ParentIndex = -1;
	int32 ChildIndex = -1;
};

//======================================================================================================================
struct FHardDistanceConstraint
{
	float TargetDistance = 0.0f;
};

//======================================================================================================================
struct FSoftDistanceConstraintInfo
{
	int32 ParentIndex = -1;
	int32 ChildIndex = -1;
};

//======================================================================================================================
struct FSoftDistanceConstraint
{
	float TargetDistance = 0.0f;

	float Compliance = 0.0f; // alpha
	float Damping = 0.0f;    // beta

	// Accumulated
	float Lambda = 0.0f;
};

//======================================================================================================================
// TODO investigate using the implicit shapes here (and for the others) - e.g. TBox/FImplicitBox3
struct FBoxShape
{
	// Optionally call once per update, if the local data have changed
	void Update(const FSimTransform& InTM, const FSimVector& InExtents)
	{
		TM = InTM;
		Extents = InExtents;
	}

	FSimTransform TM;
	FSimVector    Extents = FSimVector::ZeroVector; // Full extents
};

//======================================================================================================================
struct FCapsuleShape
{
	// Optionally call once per update, if the local data have changed
	void Update(const FSimTransform& InTM, float InLength, float InRadius)
	{
		TM = InTM;
		Length = InLength;
		Radius = InRadius;
	}

	// Axis is along +ve Z
	FSimTransform TM;
	float         Length = 0.0f; // Full length
	float         Radius = 0.0f;
};

//======================================================================================================================
struct FPlaneShape
{
	// Optionally call once per update, if the local data have changed
	void Update(const FSimTransform& InTM, const FSimVector2D& InExtents)
	{
		TM = InTM;
		Extents = InExtents;
	}

	// Plane points in +ve Z
	FSimTransform TM;
	FSimVector2D  Extents = FSimVector2D::ZeroVector;
};

//======================================================================================================================
struct FShapeCollection
{
	FSimTransform PreviousTM;
	FSimTransform TargetTM;

	// This is the current/interpolated TM
	FSimTransform TM;

	TArray<FBoxShape>     BoxShapes;
	TArray<FCapsuleShape> CapsuleShapes;
	TArray<FPlaneShape>   PlaneShapes;
};

//======================================================================================================================
// The structs above here define the individual components that could go into a particle simulation
// 
// The structs/functions below here define what components are actually being used for a particular
// particle simulation (at the moment, one based on skeletal animation/simulation).
// 
// In order to turn this into a more general-purpose solver (i.e. one that isn't 100% tailored to
// our current implementation of control rig dynamics, we will want the simulation state, and the
// functions that use it, to be more adaptable, and able to specify their own particular structs,
// constraint types, and order of operations. At that point we can consider using
// FManagedArrayCollection to store the data, and the Simulate call below can then be supplied with
// a "recipe" for how the solve should proceed, and what it expects to find.
//======================================================================================================================

//======================================================================================================================
// This contains all of the simulation state, customized for a particular simulation setup.
// Currently, the "user" needs to make and hold this, and then call the functions below in a
// specific order/way, starting with the Updates, and then calling Simulate.
//
// Each block of arrays must have matching sizes - they're separate so that data will be cached better
//======================================================================================================================
struct FSimulationState
{
	TArray<FParticleInfo>                ParticleInfos;
	TArray<FParticle>                    Particles;
	TArray<FParticleTarget>              ParticleTargets;
	TArray<FParticleCollider>            ParticleColliders;
	TArray<FParticleNoCollision>         ParticleNoCollision;  
	TArray<FParticleToParticleCollision> ParticleToParticleCollision;
	TArray<FParticleConfinement>         ParticleConfinement; // Lists which confiners confine each particle (opt-in).

	TArray<FHardDistanceConstraintInfo>  SkeletalConstraintInfos;     // Along the bones
	TArray<FHardDistanceConstraint>      SkeletalConstraints;     

	TArray<FHardDistanceConstraintInfo>  HardDistanceConstraintInfos; // User-specified
	TArray<FHardDistanceConstraint>      HardDistanceConstraints; 

	TArray<FSoftDistanceConstraintInfo>  SoftDistanceConstraintInfos; // User-specified
	TArray<FSoftDistanceConstraint>      SoftDistanceConstraints; 

	TArray<FConeLimitInfo>               ConeLimitInfos;
	TArray<FConeLimit>                   ConeLimits;

	// Particles collide with colliders unless they opt out
	TArray<FShapeCollection>             Colliders;

	// Confiners share the FShapeCollection layout with Colliders. Each confiner keeps opt-in particles
	// inside every one of its box/capsule/plane shapes (intersection semantics).
	TArray<FShapeCollection>             Confiners;
	TArray<float>                        ConfinerStrengths;

	// Previous substep time, used for time-corrected Verlet integration at frame boundaries
	float PrevStepTime = 0.0f;
};

//======================================================================================================================
struct FSolverSettings
{
	FSimVector Gravity = FSimVector::ZeroVector;
	// The total number of iterations, which includes the target solving and the constraint solving
	int32      NumIterations = 1;
	// The number of constraint (sub) iterations, processed for every one of the main iterations
	int32      NumConstraintSubIterations = 1;
	// The maximum timestep that will be applied, so if the desired step time is greater than this,
	// substeps will be taken.
	float      MaxTimeStep = 1.0f;
	// The maximum number of steps. If this is not enough, then the simulation will run slow.
	int32      MaxNumSteps = 1;
};

//======================================================================================================================
// Describes the motion of the simulation's reference frame, so the simulation can apply
// inertial pseudo-forces and air/ether drag. All quantities are in simulation space (the same
// space as the particles). Populated by the solver before calling Simulate().
//======================================================================================================================
struct FSimulationSpaceMotion
{
	FSimVector LinearVelocity = FSimVector::ZeroVector;
	FSimVector AngularVelocity = FSimVector::ZeroVector;      // rad/s
	FSimVector LinearAcceleration = FSimVector::ZeroVector;
	FSimVector AngularAcceleration = FSimVector::ZeroVector;   // rad/s^2

	// Extra air/ether velocity in simulation space. Particles with non-zero Damping are pulled
	// toward ExternalLinearVelocity + (ExternalAngularVelocity x R), in addition to the implicit
	// air/ether velocity induced by the simulation space's own motion.
	FSimVector ExternalLinearVelocity = FSimVector::ZeroVector;
	FSimVector ExternalAngularVelocity = FSimVector::ZeroVector; // rad/s

	float LinearDragMultiplier = 0.0f;   // Air/ether coupling for linear frame motion [typically 0,1]
	float AngularDragMultiplier = 0.0f;  // Air/ether coupling for angular frame motion [typically 0,1]
	float AdditionalDamping = 0.0f;      // Added to every dynamic particle's own Damping rate
	float InertialForceAmount = 0.0f;    // Overall multiplier on inertial pseudo-forces [typically 0,1]
	float LinearEulerAmount = 1.0f;      // Per-term gain on the -a_frame translational force [typically 0,1]
	float AngularEulerAmount = 1.0f;     // Per-term gain on the -alpha x r Euler force [typically 0,1]
	float CentrifugalAmount = 1.0f;      // Per-term gain on the -w x (w x r) centrifugal force [typically 0,1]
	float CoriolisAmount = 1.0f;         // Per-term gain on the -2 w x v Coriolis force [typically 0,1]

	bool bAccelerationsValid = false;
};

//======================================================================================================================
// Optionally updates a single particle's simulation properties for the current frame. If the values
// have changed, it should be called once per particle, before simulation.
//
// Index                          - particle index into the simulation state arrays
// InMass                         - particle mass (used with movement type to compute InvMass)
// InGravityMultiplier            - scales solver gravity for this particle
// InRadius                       - collision radius
// InDamping                      - air/ether drag rate in 1/time (zero disables per-particle drag)
// bInScaleDampingByInverseMass   - true scales damping by InvMass (drag-like), false leaves it mass-independent
// bInCollideWithColliders        - false disables collider contacts for this particle
//======================================================================================================================
UE_API void UpdateParticle(
	FSimulationState&                  State,
	int32                              Index,
	float                              InMass,
	float                              InGravityMultiplier,
	float                              InRadius,
	float                              InDamping,
	bool                               bInScaleDampingByInverseMass,
	bool                               bInCollideWithColliders);

//======================================================================================================================
// Updates a single particle's target position and properties for the current frame. Should be
// called once per particle per update, unless the values and target position haven't changed 
// for two updates (since velocity is calculated)
//
// Index                     - particle index into the simulation state arrays
// InvDeltaTime              - 1 / DeltaTime (or 0 if DeltaTime is near zero)
// InTargetPosition          - world-space target position from animation
// InStrength                - target tracking oscillation frequency
// InDampingRatio            - target tracking damping ratio
// InExtraDamping            - additional damping
// InTargetVelocityInfluence - how much target velocity contributes to damping
// InTargetMode              - 0 = SimSpace, 1 = directional, blend in between
// InAngleLimit              - max deviation from target direction in degrees
// InAngleLimitStrength      - oscillation frequency for the angle limit softness
// bInAccelerationMode       - target tracking only. True (default): compliance/damping cancel mass so
//                             frequency is mass-independent. False: spring/damper acts as a true force;
//                             heavier particles oscillate slower. The angle-limit constraint is always
//                             acceleration-mode regardless of this flag.
//======================================================================================================================
UE_API void UpdateParticleTarget(
	FSimulationState&                  State,
	int32                              Index,
	float                              InvDeltaTime,
	const FSimVector&                  InTargetPosition,
	float                              InStrength,
	float                              InDampingRatio,
	float                              InExtraDamping,
	float                              InTargetVelocityInfluence,
	float                              InTargetMode,
	float                              InAngleLimit,
	float                              InAngleLimitStrength,
	bool                               bInAccelerationMode);

//======================================================================================================================
// Updates a single collider for the current frame. Should be called once per collider per update,
// unless the values and target position haven't changed for two updates
//======================================================================================================================
UE_API void UpdateCollider(FSimulationState& State, int32 Index, const FSimTransform& TM);

//======================================================================================================================
// Updates a single confiner's transform for the current frame. Shares FShapeCollection's transform
// interpolation (PreviousTM -> TargetTM -> TM). Should be called once per confiner per update.
//======================================================================================================================
UE_API void UpdateConfiner(FSimulationState& State, int32 Index, const FSimTransform& TM);

//======================================================================================================================
// Updates a single skeletal (bone-chain) distance constraint's target distance from the current
// particle target positions. Call once per constraint per update when bone lengths may change (but skip if they can't).
//======================================================================================================================
UE_API void UpdateSkeletalConstraint(FSimulationState& State, int32 Index);

//======================================================================================================================
// Updates a single user-defined hard distance constraint for the current frame. Recomputes the
// target distance from particle target positions, scaled by InLengthMultiplier and offset by
// InExtraLength. Call once per constraint per update when values have changed.
//======================================================================================================================
UE_API void UpdateHardDistanceConstraint(
	FSimulationState& State, int32 Index, float InLengthMultiplier, float InExtraLength);

//======================================================================================================================
// Updates a single user-defined soft distance constraint for the current frame. Recomputes the
// target distance and compliance/damping from the given parameters. Call once per constraint per
// update when values have changed. bInAccelerationMode true (default) cancels mass from the
// compliance/damping so natural frequency is mass-independent; false makes it a force-mode spring.
//======================================================================================================================
UE_API void UpdateSoftDistanceConstraint(
	FSimulationState& State, int32 Index, float InLengthMultiplier, float InExtraLength,
	float InStrength, float InDampingRatio, float InExtraDamping, bool bInAccelerationMode);

//======================================================================================================================
// Updates a single cone limit for the current frame. Recomputes the angle limit and
// compliance/damping from the given parameters. Call once per cone limit per update when values
// have changed.
//======================================================================================================================
UE_API void UpdateConeLimit(
	FSimulationState& State, int32 Index, float InAngleDegrees, float InStrength, float InDampingRatio);

//======================================================================================================================
// Runs the simulation for one frame. Internally substeps according to MaxTimeStep/MaxNumSteps.
// The caller should have called the Update functions above to set particle targets, constraint
// distances, and collider transforms for the current frame before calling this.
// Particles should be in root-to-leaf order (see SortParticlesRootToLeaf) so that parent
// particles are processed before their children during constraint solving and readback.
// SimulationSpaceMotion is used to pass information about movement of the simulation space itself
// so that effects of non-inertial motion etc can be applied to the simulated particles.
//======================================================================================================================
UE_API void Simulate(
	FSimulationState&              SimulationState,
	const FSolverSettings&         SolverSettings,
	const FSimulationSpaceMotion&  SimulationSpaceMotion,
	float                          DeltaTime);

//======================================================================================================================
// Sorts all particle-parallel arrays in the simulation state into root-to-leaf (BFS) order.
// After sorting, ParticleInfos[i].ParentParticleIndex < i for all non-root particles.
// All stored particle index references are remapped (ParentParticleIndex, constraint
// ParentIndex/ChildIndex, and particle-to-particle collision indices), so this can be called
// at any point after particles and their parent indices have been set up.
// Returns the permutation: Result[new_index] = old_index, so the caller can reorder any
// additional parallel arrays it owns outside FSimulationState.
//======================================================================================================================
UE_API TArray<int32> SortParticlesRootToLeaf(FSimulationState& State);

//======================================================================================================================
// Trivial/inline implementations
//======================================================================================================================

//======================================================================================================================
inline void UpdateSkeletalConstraint(FSimulationState& State, const int32 Index)
{
	check(State.SkeletalConstraintInfos.IsValidIndex(Index));
	const FHardDistanceConstraintInfo& Info = State.SkeletalConstraintInfos[Index];
	check(State.ParticleInfos.IsValidIndex(Info.ParentIndex) && State.ParticleInfos.IsValidIndex(Info.ChildIndex));
	const FSimVector& ParentPos = State.ParticleInfos[Info.ParentIndex].TargetPosition;
	const FSimVector& ChildPos = State.ParticleInfos[Info.ChildIndex].TargetPosition;
	State.SkeletalConstraints[Index].TargetDistance = FSimVector::Dist(ParentPos, ChildPos);
}

//======================================================================================================================
inline void UpdateHardDistanceConstraint(
	FSimulationState& State, const int32 Index, const float InLengthMultiplier, const float InExtraLength)
{
	check(State.HardDistanceConstraintInfos.IsValidIndex(Index));
	const FHardDistanceConstraintInfo& Info = State.HardDistanceConstraintInfos[Index];
	check(State.ParticleInfos.IsValidIndex(Info.ParentIndex) && State.ParticleInfos.IsValidIndex(Info.ChildIndex));
	const FSimVector& ParentPos = State.ParticleInfos[Info.ParentIndex].TargetPosition;
	const FSimVector& ChildPos = State.ParticleInfos[Info.ChildIndex].TargetPosition;
	State.HardDistanceConstraints[Index].TargetDistance =
		FSimVector::Dist(ParentPos, ChildPos) * InLengthMultiplier + InExtraLength;
}

//======================================================================================================================
inline void UpdateSoftDistanceConstraint(
	FSimulationState& State, const int32 Index, const float InLengthMultiplier, const float InExtraLength,
	const float InStrength, const float InDampingRatio, const float InExtraDamping, const bool bInAccelerationMode)
{
	check(State.SoftDistanceConstraintInfos.IsValidIndex(Index));
	const FSoftDistanceConstraintInfo& Info = State.SoftDistanceConstraintInfos[Index];
	check(State.ParticleInfos.IsValidIndex(Info.ParentIndex) && State.ParticleInfos.IsValidIndex(Info.ChildIndex));
	const FSimVector& ParentPos = State.ParticleInfos[Info.ParentIndex].TargetPosition;
	const FSimVector& ChildPos = State.ParticleInfos[Info.ChildIndex].TargetPosition;
	FSoftDistanceConstraint& Constraint = State.SoftDistanceConstraints[Index];
	Constraint.TargetDistance = FSimVector::Dist(ParentPos, ChildPos) * InLengthMultiplier + InExtraLength;
	// Clamp Strength to >= 0: a negative value would give the same Compliance magnitude as |Strength|
	// (W is squared) but make the 2*ratio*W damping term negative, injecting energy. Strength = 0
	// disables the spring while still letting ExtraDamping apply.
	const float Strength = FMath::Max(InStrength, 0.0f);
	const float W = Strength * TWO_PI;
	const float WSquaredSafe = FMath::Max(W * W, KINDA_SMALL_NUMBER);
	if (bInAccelerationMode)
	{
		// Acceleration mode bakes InvMassSum into compliance/damping so the XPBD lambda cancels mass.
		const float InvMassSum =
			State.Particles[Info.ParentIndex].InvMass + State.Particles[Info.ChildIndex].InvMass;
		Constraint.Compliance = InvMassSum / WSquaredSafe;
		Constraint.Damping = (2.0f * InDampingRatio * W + InExtraDamping) / FMath::Max(InvMassSum, KINDA_SMALL_NUMBER);
	}
	else
	{
		// Force mode leaves mass out, so heavier (lower InvMassSum) particle pairs feel a stiffer constraint.
		Constraint.Compliance = 1.0f / WSquaredSafe;
		Constraint.Damping = 2.0f * InDampingRatio * W + InExtraDamping;
	}
}

//======================================================================================================================
inline void UpdateConeLimit(
	FSimulationState& State, const int32 Index,
	const float InAngleDegrees, const float InStrength, const float InDampingRatio)
{
	check(State.ConeLimitInfos.IsValidIndex(Index));
	const FConeLimitInfo& Info = State.ConeLimitInfos[Index];
	check(State.Particles.IsValidIndex(Info.GrandparentIndex) && State.Particles.IsValidIndex(Info.ParentIndex)
		&& State.Particles.IsValidIndex(Info.ChildIndex));
	FConeLimit& ConeLimit = State.ConeLimits[Index];
	ConeLimit.Angle = FMath::DegreesToRadians(InAngleDegrees);
	// Clamp Strength to >= 0 to stop negative values producing the same Compliance magnitude as
	// |Strength| while flipping the 2*ratio*W damping term negative (energy injection).
	const float Strength = FMath::Max(InStrength, 0.0f);
	const float W = Strength * TWO_PI;
	const float InvMassSum = State.Particles[Info.GrandparentIndex].InvMass
		+ State.Particles[Info.ParentIndex].InvMass + State.Particles[Info.ChildIndex].InvMass;
	ConeLimit.Compliance = InvMassSum / FMath::Max(W * W, KINDA_SMALL_NUMBER);
	ConeLimit.Damping = (2.0f * InDampingRatio * W) / FMath::Max(InvMassSum, KINDA_SMALL_NUMBER);
}

//======================================================================================================================
inline void UpdateCollider(FSimulationState& State, int32 Index, const FSimTransform& TM)
{
	check(State.Colliders.IsValidIndex(Index));
	RigParticleSimulation::FShapeCollection& Collider = State.Colliders[Index];
	Collider.PreviousTM = Collider.TargetTM;
	Collider.TargetTM = TM;
}

//======================================================================================================================
inline void UpdateConfiner(FSimulationState& State, int32 Index, const FSimTransform& TM)
{
	check(State.Confiners.IsValidIndex(Index));
	RigParticleSimulation::FShapeCollection& Confiner = State.Confiners[Index];
	Confiner.PreviousTM = Confiner.TargetTM;
	Confiner.TargetTM = TM;
}

//======================================================================================================================
inline void UpdateParticle(
	FSimulationState& State,
	const int32       Index,
	const float       InMass,
	const float       InGravityMultiplier,
	const float       InRadius,
	const float       InDamping,
	const bool        bInScaleDampingByInverseMass,
	const bool        bInCollideWithColliders)
{
	check(State.ParticleInfos.IsValidIndex(Index) && State.Particles.IsValidIndex(Index) &&
		State.ParticleColliders.IsValidIndex(Index));
	FParticleInfo& Info = State.ParticleInfos[Index];
	Info.GravityMultiplier = InGravityMultiplier;
	Info.Damping = InDamping;
	Info.bScaleDampingByInverseMass = bInScaleDampingByInverseMass;
	State.Particles[Index].InvMass = (Info.MovementType == ERigParticleSimulationMovementType::Simulated) ?
		1.0f / FMath::Max(InMass, KINDA_SMALL_NUMBER) : 0.0f;
	State.ParticleColliders[Index].Radius = InRadius;
	State.ParticleColliders[Index].bCollideWithColliders = bInCollideWithColliders;
}

}

#undef UE_API
