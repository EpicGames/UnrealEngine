// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ControlRigDynamicsEditor : ModuleRules
{
	public ControlRigDynamicsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ControlRig",
				"ControlRigDynamics",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"ToolMenus",
				"WorkspaceMenuStructure",
			}
		);
	}
}
