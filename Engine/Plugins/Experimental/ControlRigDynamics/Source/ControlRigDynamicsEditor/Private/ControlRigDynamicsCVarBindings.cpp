// Copyright Epic Games, Inc. All Rights Reserved.

#include "ControlRigDynamicsCVarBindings.h"

#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ControlRigDynamicsCVarBindings"

namespace ControlRigDynamicsEditor
{

//======================================================================================================================
// FCVarSimpleToggleBinding
//======================================================================================================================

//======================================================================================================================
FCVarSimpleToggleBinding::FCVarSimpleToggleBinding(const FString& InCVarName, const FText& InTooltip)
	: Tooltip(InTooltip)
{
	CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);
}

//======================================================================================================================
FCVarSimpleToggleBinding::~FCVarSimpleToggleBinding()
{
	if (CachedCVar && OnChangedHandle.IsValid())
	{
		CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
	}
}

//======================================================================================================================
void FCVarSimpleToggleBinding::Initialize(TWeakPtr<SWidget> InOwnerWidget)
{
	OwnerWidget = InOwnerWidget;
	if (CachedCVar && !OnChangedHandle.IsValid())
	{
		OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
			AsShared(), &FCVarSimpleToggleBinding::HandleCVarChanged);
	}
}

//======================================================================================================================
TSharedRef<SWidget> FCVarSimpleToggleBinding::BuildOverrideCell()
{
	// Simple toggles have no override — return an empty spacer so the grid column stays aligned.
	return SNew(SSpacer);
}

//======================================================================================================================
TSharedRef<SWidget> FCVarSimpleToggleBinding::BuildValueCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarSimpleToggleBinding::GetValueCheckState)
		.OnCheckStateChanged(this, &FCVarSimpleToggleBinding::OnValueCheckStateChanged)
		.IsEnabled(CachedCVar != nullptr)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
void FCVarSimpleToggleBinding::HandleCVarChanged(IConsoleVariable* /*InVariable*/)
{
	if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
	{
		Owner->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

//======================================================================================================================
ECheckBoxState FCVarSimpleToggleBinding::GetValueCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	return CachedCVar->GetInt() != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarSimpleToggleBinding::OnValueCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	const int32 NewValue = (NewState == ECheckBoxState::Checked) ? 1 : 0;
	CachedCVar->Set(NewValue, ECVF_SetByConsole);
}


//======================================================================================================================
// FCVarOverrideToggleBinding
//======================================================================================================================

//======================================================================================================================
FCVarOverrideToggleBinding::FCVarOverrideToggleBinding(const FString& InCVarName, const FText& InTooltip)
	: Tooltip(InTooltip)
{
	CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);
	if (CachedCVar)
	{
		const int32 CurrentValue = CachedCVar->GetInt();
		if (CurrentValue == 0 || CurrentValue == 1)
		{
			PendingValue = CurrentValue;
		}
	}
}

//======================================================================================================================
FCVarOverrideToggleBinding::~FCVarOverrideToggleBinding()
{
	if (CachedCVar && OnChangedHandle.IsValid())
	{
		CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
	}
}

//======================================================================================================================
void FCVarOverrideToggleBinding::Initialize(TWeakPtr<SWidget> InOwnerWidget)
{
	OwnerWidget = InOwnerWidget;
	if (CachedCVar && !OnChangedHandle.IsValid())
	{
		OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
			AsShared(), &FCVarOverrideToggleBinding::HandleCVarChanged);
	}
}

//======================================================================================================================
TSharedRef<SWidget> FCVarOverrideToggleBinding::BuildOverrideCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarOverrideToggleBinding::GetApplyCheckState)
		.OnCheckStateChanged(this, &FCVarOverrideToggleBinding::OnApplyCheckStateChanged)
		.IsEnabled(CachedCVar != nullptr)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
TSharedRef<SWidget> FCVarOverrideToggleBinding::BuildValueCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarOverrideToggleBinding::GetValueCheckState)
		.OnCheckStateChanged(this, &FCVarOverrideToggleBinding::OnValueCheckStateChanged)
		.IsEnabled(this, &FCVarOverrideToggleBinding::IsValueToggleEnabled)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
void FCVarOverrideToggleBinding::HandleCVarChanged(IConsoleVariable* /*InVariable*/)
{
	if (CachedCVar)
	{
		const int32 CurrentValue = CachedCVar->GetInt();
		if (CurrentValue == 0 || CurrentValue == 1)
		{
			PendingValue = CurrentValue;
		}
	}
	if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
	{
		Owner->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

//======================================================================================================================
ECheckBoxState FCVarOverrideToggleBinding::GetApplyCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	return CachedCVar->GetInt() >= 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarOverrideToggleBinding::OnApplyCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	if (NewState == ECheckBoxState::Checked)
	{
		CachedCVar->Set(PendingValue, ECVF_SetByConsole);
	}
	else
	{
		CachedCVar->Set(-1, ECVF_SetByConsole);
	}
}

//======================================================================================================================
ECheckBoxState FCVarOverrideToggleBinding::GetValueCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	const int32 CurrentValue = CachedCVar->GetInt();
	const int32 ValueToShow = (CurrentValue >= 0) ? CurrentValue : PendingValue;
	return ValueToShow != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

//======================================================================================================================
void FCVarOverrideToggleBinding::OnValueCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	PendingValue = (NewState == ECheckBoxState::Checked) ? 1 : 0;
	if (CachedCVar->GetInt() >= 0)
	{
		CachedCVar->Set(PendingValue, ECVF_SetByConsole);
	}
}

//======================================================================================================================
bool FCVarOverrideToggleBinding::IsValueToggleEnabled() const
{
	return CachedCVar != nullptr && CachedCVar->GetInt() >= 0;
}


//======================================================================================================================
// FCVarEnumDropdownBinding
//======================================================================================================================

//======================================================================================================================
FCVarEnumDropdownBinding::FCVarEnumDropdownBinding(
	const FString& InCVarName,
	const FText& InTooltip,
	UEnum* InEnum,
	const FString& InDefaultWhenApplied)
	: Tooltip(InTooltip)
	, Enum(InEnum)
	, DefaultWhenApplied(InDefaultWhenApplied)
	, PendingValue(InDefaultWhenApplied)
{
	CachedCVar = IConsoleManager::Get().FindConsoleVariable(*InCVarName);

	// Populate combo options from the enum. Skip the auto-generated _MAX sentinel.
	if (Enum)
	{
		const int32 NumEntries = Enum->NumEnums();
		for (int32 Index = 0; Index < NumEntries; ++Index)
		{
			const FString EntryShortName = Enum->GetNameStringByIndex(Index);
			if (EntryShortName.EndsWith(TEXT("_MAX")))
			{
				continue;
			}
			Options.Add(MakeShared<FString>(EntryShortName));
		}
	}

	if (CachedCVar)
	{
		const FString CurrentValue = CachedCVar->GetString();
		if (!CurrentValue.IsEmpty() && FindOptionForString(CurrentValue).IsValid())
		{
			PendingValue = CurrentValue;
		}
	}
}

//======================================================================================================================
FCVarEnumDropdownBinding::~FCVarEnumDropdownBinding()
{
	if (CachedCVar && OnChangedHandle.IsValid())
	{
		CachedCVar->OnChangedDelegate().Remove(OnChangedHandle);
	}
}

//======================================================================================================================
void FCVarEnumDropdownBinding::Initialize(TWeakPtr<SWidget> InOwnerWidget)
{
	OwnerWidget = InOwnerWidget;
	if (CachedCVar && !OnChangedHandle.IsValid())
	{
		OnChangedHandle = CachedCVar->OnChangedDelegate().AddSP(
			AsShared(), &FCVarEnumDropdownBinding::HandleCVarChanged);
	}
}

//======================================================================================================================
TSharedRef<SWidget> FCVarEnumDropdownBinding::BuildOverrideCell()
{
	return SNew(SCheckBox)
		.IsChecked(this, &FCVarEnumDropdownBinding::GetApplyCheckState)
		.OnCheckStateChanged(this, &FCVarEnumDropdownBinding::OnApplyCheckStateChanged)
		.IsEnabled(CachedCVar != nullptr)
		.ToolTipText(Tooltip);
}

//======================================================================================================================
TSharedRef<SWidget> FCVarEnumDropdownBinding::BuildValueCell()
{
	const TSharedPtr<FString> InitialSelection = FindOptionForString(PendingValue);

	return SAssignNew(ComboWidget, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&Options)
		.InitiallySelectedItem(InitialSelection)
		.OnGenerateWidget(this, &FCVarEnumDropdownBinding::OnGenerateComboItem)
		.OnSelectionChanged(this, &FCVarEnumDropdownBinding::OnComboSelectionChanged)
		.IsEnabled(this, &FCVarEnumDropdownBinding::IsValueComboEnabled)
		.ToolTipText(Tooltip)
		[
			SNew(STextBlock)
			.Text(this, &FCVarEnumDropdownBinding::GetComboSelectedText)
		];
}

//======================================================================================================================
void FCVarEnumDropdownBinding::HandleCVarChanged(IConsoleVariable* /*InVariable*/)
{
	if (CachedCVar)
	{
		const FString CurrentValue = CachedCVar->GetString();
		if (!CurrentValue.IsEmpty())
		{
			if (TSharedPtr<FString> Match = FindOptionForString(CurrentValue))
			{
				PendingValue = *Match;
				if (ComboWidget.IsValid())
				{
					ComboWidget->SetSelectedItem(Match);
				}
			}
		}
	}
	if (TSharedPtr<SWidget> Owner = OwnerWidget.Pin())
	{
		Owner->Invalidate(EInvalidateWidgetReason::Paint);
	}
}

//======================================================================================================================
ECheckBoxState FCVarEnumDropdownBinding::GetApplyCheckState() const
{
	if (!CachedCVar)
	{
		return ECheckBoxState::Unchecked;
	}
	return CachedCVar->GetString().IsEmpty() ? ECheckBoxState::Unchecked : ECheckBoxState::Checked;
}

//======================================================================================================================
void FCVarEnumDropdownBinding::OnApplyCheckStateChanged(ECheckBoxState NewState)
{
	if (!CachedCVar)
	{
		return;
	}
	if (NewState == ECheckBoxState::Checked)
	{
		const FString ValueToSet = PendingValue.IsEmpty() ? DefaultWhenApplied : PendingValue;
		CachedCVar->Set(*ValueToSet, ECVF_SetByConsole);
	}
	else
	{
		CachedCVar->Set(TEXT(""), ECVF_SetByConsole);
	}
}

//======================================================================================================================
bool FCVarEnumDropdownBinding::IsValueComboEnabled() const
{
	return CachedCVar != nullptr && !CachedCVar->GetString().IsEmpty();
}

//======================================================================================================================
TSharedRef<SWidget> FCVarEnumDropdownBinding::OnGenerateComboItem(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(InItem.IsValid() ? *InItem : FString()));
}

//======================================================================================================================
void FCVarEnumDropdownBinding::OnComboSelectionChanged(
	TSharedPtr<FString> InItem, ESelectInfo::Type SelectInfo)
{
	// Ignore programmatic selection changes that originate from our own HandleCVarChanged.
	if (SelectInfo == ESelectInfo::Direct || !InItem.IsValid())
	{
		return;
	}
	PendingValue = *InItem;
	if (CachedCVar && !CachedCVar->GetString().IsEmpty())
	{
		CachedCVar->Set(*PendingValue, ECVF_SetByConsole);
	}
}

//======================================================================================================================
FText FCVarEnumDropdownBinding::GetComboSelectedText() const
{
	if (!CachedCVar)
	{
		return FText::GetEmpty();
	}
	const FString CurrentValue = CachedCVar->GetString();
	const FString ValueToShow = CurrentValue.IsEmpty() ? PendingValue : CurrentValue;
	return FText::FromString(ValueToShow);
}

//======================================================================================================================
TSharedPtr<FString> FCVarEnumDropdownBinding::FindOptionForString(const FString& InValue) const
{
	for (const TSharedPtr<FString>& Option : Options)
	{
		if (Option.IsValid() && *Option == InValue)
		{
			return Option;
		}
	}
	return nullptr;
}

} // namespace ControlRigDynamicsEditor

#undef LOCTEXT_NAMESPACE
