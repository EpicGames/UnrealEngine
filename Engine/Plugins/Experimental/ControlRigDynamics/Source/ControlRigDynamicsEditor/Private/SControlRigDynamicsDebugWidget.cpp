// Copyright Epic Games, Inc. All Rights Reserved.

#include "SControlRigDynamicsDebugWidget.h"

#include "ControlRigDynamicsCVarBindings.h"
#include "RigDynamicsData.h"

#include "HAL/IConsoleManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "UObject/Class.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SControlRigDynamicsDebugWidget"

using namespace ControlRigDynamicsEditor;

namespace ControlRigDynamicsDebugWidgetPrivate
{
	// Kept in a single place so the layout and the Clear Overrides action agree on the list.
	static const TCHAR* const AllOverrideCVarNames[] =
	{
		TEXT("ControlRig.Dynamics.MaxTimeStepOverride"),
		TEXT("ControlRig.Dynamics.MaxNumStepsOverride"),
		TEXT("ControlRig.Dynamics.ShowParticlesOverride"),
		TEXT("ControlRig.Dynamics.ShowCollidersOverride"),
		TEXT("ControlRig.Dynamics.ShowConfinersOverride"),
		TEXT("ControlRig.Dynamics.ShowSkeletalConstraintsOverride"),
		TEXT("ControlRig.Dynamics.ShowHardConstraintsOverride"),
		TEXT("ControlRig.Dynamics.ShowSoftConstraintsOverride"),
		TEXT("ControlRig.Dynamics.ShowAngleLimitsOverride"),
		TEXT("ControlRig.Dynamics.ShowConeLimitsOverride"),
		TEXT("ControlRig.Dynamics.ShowForceFieldsOverride"),
		TEXT("ControlRig.Dynamics.ForceFieldDebugScaleOverride"),
	};

	constexpr int32 ColOverride = 0;
	constexpr int32 ColName     = 1;
	constexpr int32 ColValue    = 2;
	constexpr int32 NumColumns  = 3;

	//==================================================================================================================
	static TSharedRef<SWidget> MakeSectionHeader(const FText& InText)
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(6.0f, 4.0f))
			[
				SNew(STextBlock)
				.Text(InText)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
			];
	}

	//==================================================================================================================
	static TSharedRef<SWidget> MakeLabelCell(const FText& InText, const FText& InTooltip)
	{
		return SNew(STextBlock)
			.Text(InText)
			.ToolTipText(InTooltip);
	}
}

//======================================================================================================================
void SControlRigDynamicsDebugWidget::Construct(const FArguments& /*InArgs*/)
{
	using namespace ControlRigDynamicsDebugWidgetPrivate;

	// --- Tooltips ---------------------------------------------------------------------------------------------

	const FText EnableStepSolverTooltip = LOCTEXT("EnableStepSolverTooltip",
		"When off, every solver step is skipped. Useful for pausing dynamics while debugging.");
	const FText MaxTimeStepTooltip = LOCTEXT("MaxTimeStepOverrideTooltip",
		"When applied, overrides the solver's MaxTimeStep (seconds). Controls how large a single substep is allowed to be.");
	const FText MaxNumStepsTooltip = LOCTEXT("MaxNumStepsOverrideTooltip",
		"When applied, overrides the solver's MaxNumSteps (maximum substeps per frame).");
	const FText AllowVisualizationTooltip = LOCTEXT("AllowVisualizationTooltip",
		"Master allow-switch for dynamics debug visualization. When off, all debug drawing is suppressed regardless of the rig's settings (useful for profiling). When on, drawing is gated by the rig's individual show flags.");
	const FText ShowParticlesTooltip = LOCTEXT("ShowParticlesTooltip",
		"Overrides whether particle spheres are drawn.");
	const FText ShowCollidersTooltip = LOCTEXT("ShowCollidersTooltip",
		"Overrides whether collider shapes are drawn.");
	const FText ShowConfinersTooltip = LOCTEXT("ShowConfinersTooltip",
		"Overrides whether confiner shapes are drawn.");
	const FText ShowSkeletalConstraintsTooltip = LOCTEXT("ShowSkeletalConstraintsTooltip",
		"Overrides whether skeletal parent/child distance constraints are drawn.");
	const FText ShowHardConstraintsTooltip = LOCTEXT("ShowHardConstraintsTooltip",
		"Overrides whether user-authored hard distance constraints are drawn.");
	const FText ShowSoftConstraintsTooltip = LOCTEXT("ShowSoftConstraintsTooltip",
		"Overrides whether user-authored soft distance constraints are drawn.");
	const FText ShowAngleLimitsTooltip = LOCTEXT("ShowAngleLimitsTooltip",
		"Overrides whether per-particle angle-limit cones are drawn.");
	const FText ShowConeLimitsTooltip = LOCTEXT("ShowConeLimitsTooltip",
		"Overrides whether triple-particle cone-limit cones are drawn.");
	const FText ShowForceFieldsTooltip = LOCTEXT("ShowForceFieldsTooltip",
		"Overrides per-instance force-field debug drawing (the field node's bDrawDebug pin). Affects both the ellipsoid wireframe and the per-particle arrows.");
	const FText ForceFieldDebugScaleTooltip = LOCTEXT("ForceFieldDebugScaleTooltip",
		"When applied, overrides every force-field's DebugForceScale pin. Use to globally rescale arrow lengths without editing individual nodes.");
	const FText ParticleValueDisplayTooltip = LOCTEXT("ParticleValueDisplayTooltip",
		"Overrides which particle property the numeric overlay prints (None/Radius/Mass/Drag/...).");

	const FText EnableDrawInterfaceInGameTooltip = LOCTEXT("EnableDrawInterfaceInGameTooltip",
		"Enables/disables general Control Rig debug drawing during PIE and in-game. "
		"Required to be on for any of the Visualization toggles above to take effect at runtime.");
	const FText AnimNodeControlRigDebugTooltip = LOCTEXT("AnimNodeControlRigDebugTooltip",
		"Enables/disables debug drawing for AnimNode_ControlRigBase. Required for the Visualization toggles above to take effect "
		"when the rig is driven by an AnimGraph node. Only registered in builds with ENABLE_ANIM_DEBUG.");

	// --- Create bindings --------------------------------------------------------------------------------------

	EnableStepSolverBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.Dynamics.EnableStepSolver"), EnableStepSolverTooltip);
	MaxTimeStepBinding = MakeShared<FCVarOverrideNumericBinding<float>>(
		TEXT("ControlRig.Dynamics.MaxTimeStepOverride"), MaxTimeStepTooltip, 1.0f / 60.0f, 0.0f);
	MaxNumStepsBinding = MakeShared<FCVarOverrideNumericBinding<int32>>(
		TEXT("ControlRig.Dynamics.MaxNumStepsOverride"), MaxNumStepsTooltip, 4, 0);
	AllowVisualizationBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.Dynamics.AllowVisualization"), AllowVisualizationTooltip);
	ShowParticlesBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowParticlesOverride"), ShowParticlesTooltip);
	ShowCollidersBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowCollidersOverride"), ShowCollidersTooltip);
	ShowConfinersBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowConfinersOverride"), ShowConfinersTooltip);
	ShowSkeletalConstraintsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowSkeletalConstraintsOverride"), ShowSkeletalConstraintsTooltip);
	ShowHardConstraintsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowHardConstraintsOverride"), ShowHardConstraintsTooltip);
	ShowSoftConstraintsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowSoftConstraintsOverride"), ShowSoftConstraintsTooltip);
	ShowAngleLimitsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowAngleLimitsOverride"), ShowAngleLimitsTooltip);
	ShowConeLimitsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowConeLimitsOverride"), ShowConeLimitsTooltip);
	ShowForceFieldsBinding = MakeShared<FCVarOverrideToggleBinding>(
		TEXT("ControlRig.Dynamics.ShowForceFieldsOverride"), ShowForceFieldsTooltip);
	ForceFieldDebugScaleBinding = MakeShared<FCVarOverrideNumericBinding<float>>(
		TEXT("ControlRig.Dynamics.ForceFieldDebugScaleOverride"), ForceFieldDebugScaleTooltip, 1.0f, 0.0f);
	ParticleValueDisplayBinding = MakeShared<FCVarEnumDropdownBinding>(
		TEXT("ControlRig.Dynamics.ParticleValueDisplayOverride"), ParticleValueDisplayTooltip,
		StaticEnum<ERigDynamicsParticleValueDisplay>(), TEXT("Mass"));

	EnableDrawInterfaceInGameBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("ControlRig.EnableDrawInterfaceInGame"), EnableDrawInterfaceInGameTooltip);
	AnimNodeControlRigDebugBinding = MakeShared<FCVarSimpleToggleBinding>(
		TEXT("a.AnimNode.ControlRig.Debug"), AnimNodeControlRigDebugTooltip);

	// --- Build grid -------------------------------------------------------------------------------------------

	TSharedRef<SGridPanel> Grid = SNew(SGridPanel).FillColumn(ColName, 1.0f);

	const FMargin CellPadding(4.0f, 3.0f);
	int32 Row = 0;

	auto AddSectionHeader = [&Grid, &Row](const FText& InText)
	{
		Grid->AddSlot(ColOverride, Row)
			.ColumnSpan(NumColumns)
			.Padding(FMargin(0.0f, 6.0f, 0.0f, 2.0f))
			[
				MakeSectionHeader(InText)
			];
		++Row;
	};

	auto AddBindingRow = [&Grid, &Row, CellPadding](
		const TSharedRef<SWidget>& OverrideCell,
		const FText& Label,
		const FText& Tooltip,
		const TSharedRef<SWidget>& ValueCell)
	{
		Grid->AddSlot(ColOverride, Row)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				OverrideCell
			];
		Grid->AddSlot(ColName, Row)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				MakeLabelCell(Label, Tooltip)
			];
		Grid->AddSlot(ColValue, Row)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(CellPadding)
			[
				ValueCell
			];
		++Row;
	};

	// --- Simulation -------------------------------------------------------------------------------------------

	AddSectionHeader(LOCTEXT("SimulationSection", "Simulation"));

	AddBindingRow(
		EnableStepSolverBinding->BuildOverrideCell(),
		LOCTEXT("EnableStepSolverLabel", "Enable Step Solver"), EnableStepSolverTooltip,
		EnableStepSolverBinding->BuildValueCell());

	AddBindingRow(
		MaxTimeStepBinding->BuildOverrideCell(),
		LOCTEXT("MaxTimeStepOverrideLabel", "Max Time Step"), MaxTimeStepTooltip,
		MaxTimeStepBinding->BuildValueCell());

	AddBindingRow(
		MaxNumStepsBinding->BuildOverrideCell(),
		LOCTEXT("MaxNumStepsOverrideLabel", "Max Num Steps"), MaxNumStepsTooltip,
		MaxNumStepsBinding->BuildValueCell());

	// --- Visualization ----------------------------------------------------------------------------------------

	AddSectionHeader(LOCTEXT("VisualizationSection", "Visualization"));

	AddBindingRow(
		AllowVisualizationBinding->BuildOverrideCell(),
		LOCTEXT("AllowVisualizationLabel", "Allow Visualization"), AllowVisualizationTooltip,
		AllowVisualizationBinding->BuildValueCell());

	AddBindingRow(
		ShowParticlesBinding->BuildOverrideCell(),
		LOCTEXT("ShowParticlesLabel", "Show Particles"), ShowParticlesTooltip,
		ShowParticlesBinding->BuildValueCell());

	AddBindingRow(
		ShowCollidersBinding->BuildOverrideCell(),
		LOCTEXT("ShowCollidersLabel", "Show Colliders"), ShowCollidersTooltip,
		ShowCollidersBinding->BuildValueCell());

	AddBindingRow(
		ShowConfinersBinding->BuildOverrideCell(),
		LOCTEXT("ShowConfinersLabel", "Show Confiners"), ShowConfinersTooltip,
		ShowConfinersBinding->BuildValueCell());

	AddBindingRow(
		ShowSkeletalConstraintsBinding->BuildOverrideCell(),
		LOCTEXT("ShowSkeletalConstraintsLabel", "Show Skeletal Constraints"), ShowSkeletalConstraintsTooltip,
		ShowSkeletalConstraintsBinding->BuildValueCell());

	AddBindingRow(
		ShowHardConstraintsBinding->BuildOverrideCell(),
		LOCTEXT("ShowHardConstraintsLabel", "Show Hard Constraints"), ShowHardConstraintsTooltip,
		ShowHardConstraintsBinding->BuildValueCell());

	AddBindingRow(
		ShowSoftConstraintsBinding->BuildOverrideCell(),
		LOCTEXT("ShowSoftConstraintsLabel", "Show Soft Constraints"), ShowSoftConstraintsTooltip,
		ShowSoftConstraintsBinding->BuildValueCell());

	AddBindingRow(
		ShowAngleLimitsBinding->BuildOverrideCell(),
		LOCTEXT("ShowAngleLimitsLabel", "Show Angle Limits"), ShowAngleLimitsTooltip,
		ShowAngleLimitsBinding->BuildValueCell());

	AddBindingRow(
		ShowConeLimitsBinding->BuildOverrideCell(),
		LOCTEXT("ShowConeLimitsLabel", "Show Cone Limits"), ShowConeLimitsTooltip,
		ShowConeLimitsBinding->BuildValueCell());

	AddBindingRow(
		ShowForceFieldsBinding->BuildOverrideCell(),
		LOCTEXT("ShowForceFieldsLabel", "Show Force Fields"), ShowForceFieldsTooltip,
		ShowForceFieldsBinding->BuildValueCell());

	AddBindingRow(
		ForceFieldDebugScaleBinding->BuildOverrideCell(),
		LOCTEXT("ForceFieldDebugScaleLabel", "Force Field Debug Scale"), ForceFieldDebugScaleTooltip,
		ForceFieldDebugScaleBinding->BuildValueCell());

	AddBindingRow(
		ParticleValueDisplayBinding->BuildOverrideCell(),
		LOCTEXT("ParticleValueDisplayLabel", "Particle Value Display"), ParticleValueDisplayTooltip,
		ParticleValueDisplayBinding->BuildValueCell());

	// --- In-Game --------------------------------------------------------------------------------------------
	// Engine-wide gates that suppress all Control Rig debug drawing in PIE / shipped game. These are independent
	// of the rig's Visualization flags above; both have to be on for any of the rows above to take effect at runtime.

	AddSectionHeader(LOCTEXT("InGameSection", "In-Game"));

	AddBindingRow(
		EnableDrawInterfaceInGameBinding->BuildOverrideCell(),
		LOCTEXT("EnableDrawInterfaceInGameLabel", "ControlRig.EnableDrawInterfaceInGame"), EnableDrawInterfaceInGameTooltip,
		EnableDrawInterfaceInGameBinding->BuildValueCell());

	AddBindingRow(
		AnimNodeControlRigDebugBinding->BuildOverrideCell(),
		LOCTEXT("AnimNodeControlRigDebugLabel", "a.AnimNode.ControlRig.Debug"), AnimNodeControlRigDebugTooltip,
		AnimNodeControlRigDebugBinding->BuildValueCell());

	// --- Assemble top-level layout ---------------------------------------------------------------------------

	ChildSlot
	.Padding(4.0f)
	[
		SNew(SBox)
		.MinDesiredWidth(460.0f)
		[
			SNew(SVerticalBox)

			// Scrollable area: the grid. Takes all remaining vertical space and scrolls when squished.
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					Grid
				]
			]

			// Separator and Clear Overrides button stay pinned at the bottom, outside the scroll area.
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 8.0f)
			[
				SNew(SSeparator)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(4.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearOverridesButton", "Clear Overrides"))
				.ToolTipText(LOCTEXT("ClearOverridesTooltip",
					"Clears the override checkboxes, and resets only the values that don't have an override checkbox to their defaults. "
					"Values guarded by an override checkbox are preserved, so re-enabling the checkbox restores what you had."))
				.OnClicked(this, &SControlRigDynamicsDebugWidget::OnClearOverridesClicked)
			]
		]
	];

	// --- Hook OnChangedDelegate listeners now that SharedThis is valid ---------------------------------------

	const TWeakPtr<SWidget> WeakSelf = SharedThis(this);

	EnableStepSolverBinding->Initialize(WeakSelf);
	MaxTimeStepBinding->Initialize(WeakSelf);
	MaxNumStepsBinding->Initialize(WeakSelf);
	AllowVisualizationBinding->Initialize(WeakSelf);
	ShowParticlesBinding->Initialize(WeakSelf);
	ShowCollidersBinding->Initialize(WeakSelf);
	ShowConfinersBinding->Initialize(WeakSelf);
	ShowSkeletalConstraintsBinding->Initialize(WeakSelf);
	ShowHardConstraintsBinding->Initialize(WeakSelf);
	ShowSoftConstraintsBinding->Initialize(WeakSelf);
	ShowAngleLimitsBinding->Initialize(WeakSelf);
	ShowConeLimitsBinding->Initialize(WeakSelf);
	ShowForceFieldsBinding->Initialize(WeakSelf);
	ForceFieldDebugScaleBinding->Initialize(WeakSelf);
	ParticleValueDisplayBinding->Initialize(WeakSelf);
	EnableDrawInterfaceInGameBinding->Initialize(WeakSelf);
	AnimNodeControlRigDebugBinding->Initialize(WeakSelf);
}

//======================================================================================================================
FReply SControlRigDynamicsDebugWidget::OnClearOverridesClicked()
{
	using namespace ControlRigDynamicsDebugWidgetPrivate;

	IConsoleManager& ConsoleManager = IConsoleManager::Get();

	for (const TCHAR* CVarName : AllOverrideCVarNames)
	{
		if (IConsoleVariable* CVar = ConsoleManager.FindConsoleVariable(CVarName))
		{
			CVar->Set(TEXT("-1"), ECVF_SetByConsole);
		}
	}

	if (IConsoleVariable* EnableStepSolver = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.Dynamics.EnableStepSolver")))
	{
		EnableStepSolver->Set(TEXT("1"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* AllowVisualization = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.Dynamics.AllowVisualization")))
	{
		AllowVisualization->Set(TEXT("1"), ECVF_SetByConsole);
	}

	// String-typed override uses "" as the "no override" sentinel rather than -1.
	if (IConsoleVariable* ParticleValueDisplay = ConsoleManager.FindConsoleVariable(
			TEXT("ControlRig.Dynamics.ParticleValueDisplayOverride")))
	{
		ParticleValueDisplay->Set(TEXT(""), ECVF_SetByConsole);
	}

	if (IConsoleVariable* EnableDrawInGame = ConsoleManager.FindConsoleVariable(TEXT("ControlRig.EnableDrawInterfaceInGame")))
	{
		EnableDrawInGame->Set(TEXT("0"), ECVF_SetByConsole);
	}

	if (IConsoleVariable* AnimNodeDebug = ConsoleManager.FindConsoleVariable(TEXT("a.AnimNode.ControlRig.Debug")))
	{
		AnimNodeDebug->Set(TEXT("0"), ECVF_SetByConsole);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
