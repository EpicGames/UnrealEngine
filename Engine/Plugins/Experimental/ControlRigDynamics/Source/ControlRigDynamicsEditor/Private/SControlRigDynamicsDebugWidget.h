// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ControlRigDynamicsCVarBindings.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

//======================================================================================================================
// Top-level debug widget for ControlRigDynamics. Lays out the CVar controls in a 3-column grid
// (Override | Name | Value) so the Apply checkboxes and value widgets align neatly across rows.
//======================================================================================================================
class SControlRigDynamicsDebugWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SControlRigDynamicsDebugWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FReply OnClearOverridesClicked();

	// Binding lifetime matches the widget. Kept as members so OnChangedDelegate bindings stay alive.
	// The binding types are namespaced (ControlRigDynamicsEditor::) to avoid duplicate-symbol clashes
	// with the parallel set declared in ControlRigPhysicsEditor.
	TSharedPtr<ControlRigDynamicsEditor::FCVarSimpleToggleBinding>           EnableStepSolverBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideNumericBinding<float>> MaxTimeStepBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideNumericBinding<int32>> MaxNumStepsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarSimpleToggleBinding>           AllowVisualizationBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowParticlesBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowCollidersBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowConfinersBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowSkeletalConstraintsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowHardConstraintsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowSoftConstraintsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowAngleLimitsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowConeLimitsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideToggleBinding>         ShowForceFieldsBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarOverrideNumericBinding<float>> ForceFieldDebugScaleBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarEnumDropdownBinding>           ParticleValueDisplayBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarSimpleToggleBinding>           EnableDrawInterfaceInGameBinding;
	TSharedPtr<ControlRigDynamicsEditor::FCVarSimpleToggleBinding>           AnimNodeControlRigDebugBinding;
};
